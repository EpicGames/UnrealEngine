{
	"FileVersion" : 3,
	
	"FriendlyName" : "Messaging Debugger",
	"Version" : 1,
	"VersionName" : "1.1",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"Description" : "Provides a visual debugger for the messaging sub-system.",
	"Category" : "Messaging",
	"EnabledByDefault" : false,
	"IsBetaVersion": true,
	"SupportedPrograms":
	[
		"SpatialMetricsProfiler"
	],
	"Modules" :
	[
		{
			"Name" : "MessagingDebugger",
			"Type" : "UncookedOnly"
		}
	],

	"CanContainContent" : false
}