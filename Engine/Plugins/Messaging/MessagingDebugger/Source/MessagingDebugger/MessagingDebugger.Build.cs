// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MessagingDebugger : ModuleRules
{
	public MessagingDebugger(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"InputCore",
				"Serialization",
				"Slate",
				"SlateCore",
			});

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Messaging",
			});

		PrivateIncludePaths.AddRange(
			new string[] {
				"MessagingDebugger/Private/Models",
				"MessagingDebugger/Private/Styles",
				"MessagingDebugger/Private/Widgets",
				"MessagingDebugger/Private/Widgets/Breakpoints",
				"MessagingDebugger/Private/Widgets/EndpointDetails",
				"MessagingDebugger/Private/Widgets/Endpoints",
				"MessagingDebugger/Private/Widgets/Graph",
				"MessagingDebugger/Private/Widgets/History",
				"MessagingDebugger/Private/Widgets/Interceptors",
				"MessagingDebugger/Private/Widgets/MessageData",
				"MessagingDebugger/Private/Widgets/MessageDetails",
				"MessagingDebugger/Private/Widgets/Types",
				"MessagingDebugger/Private/Widgets/Toolbar",
			});

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"PropertyEditor",
					"WorkspaceMenuStructure",
				});
		}
	}
}
