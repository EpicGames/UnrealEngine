// Copyright Epic Games, Inc. All Rights Reserved.

#include "Features/IModularFeatures.h"
#include "IMessageBus.h"
#include "IMessagingModule.h"
#include "Models/MessagingDebuggerCommands.h"
#include "Styles/MessagingDebuggerStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SMessagingDebugger.h"
#include "Widgets/Text/STextBlock.h"

#if WITH_EDITOR
	#include "WorkspaceMenuStructure.h"
	#include "WorkspaceMenuStructureModule.h"
#endif


#define LOCTEXT_NAMESPACE "FMessagingDebuggerModule"

static const FName MessagingDebuggerTabName("MessagingDebugger");
static const FName MessagingDebuggerDocumentTabName("MessagingDebuggerDocumentTab");

/**
 * Implements the MessagingDebugger module.
 */
class FMessagingDebuggerModule
	: public IModuleInterface
	, public IModularFeature
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override
	{
		Style = MakeShared<FMessagingDebuggerStyle>();

		FMessagingDebuggerCommands::Register();
		IModularFeatures::Get().RegisterModularFeature("MessagingDebugger", this);
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(MessagingDebuggerTabName, FOnSpawnTab::CreateRaw(this, &FMessagingDebuggerModule::SpawnMessagingDebuggerTab))
			.SetDisplayName(LOCTEXT("TabTitle", "Messaging Debugger"))
#if WITH_EDITOR
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory())
#else

#endif
			.SetIcon(FSlateIcon(Style->GetStyleSetName(), "MessagingDebuggerTabIcon"))
			.SetTooltipText(LOCTEXT("TooltipText", "Visual debugger for the messaging sub-system."));
	}

	virtual void ShutdownModule() override
	{
		// if the messaging module is still loaded on shutdown remove our delegate hooks if still any
		if (IMessagingModule* MessagingModule = FModuleManager::GetModulePtr<IMessagingModule>("Messaging"))
		{
			RemoveMessagingHooks(*MessagingModule);
		}
		
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(MessagingDebuggerTabName);
		IModularFeatures::Get().UnregisterModularFeature("MessagingDebugger", this);
		FMessagingDebuggerCommands::Unregister();
	}

private:
	/**
	 * Create a new messaging debugger tab.
	 *
	 * @param SpawnTabArgs The arguments for the tab to spawn.
	 * @return The spawned tab.
	 */
	TSharedRef<SDockTab> SpawnMessagingDebuggerTab(const FSpawnTabArgs&)
	{
		MajorDebuggerTab = SNew(SDockTab)
			.TabRole(ETabRole::MajorTab);			
		MajorDebuggerTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FMessagingDebuggerModule::OnDebuggerTabClosed));

		// Create document tab for each existing bus
		TArray<TSharedRef<IMessageBus, ESPMode::ThreadSafe>> AllBuses = IMessagingModule::Get().GetAllBuses();
		if (AllBuses.Num() > 0)
		{
			for (const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& MessageBus : AllBuses)
			{
				AddDebuggerTab(MessageBus);
			}
		}
		else
		{
			TSharedPtr<SWidget> TabContent = SNew(STextBlock)
				.Text(LOCTEXT("MessagingSystemNoBusError", "No messagging bus currently exists."));
		}

		// Register to message bus startup/shutdown delegate
		AddMessagingHooks(IMessagingModule::Get());
		return MajorDebuggerTab.ToSharedRef();
	}

	void AddMessagingHooks(IMessagingModule& MessagingModule)
	{
		BusStartupHandle = MessagingModule.OnMessageBusStartup().AddLambda([this](TWeakPtr<IMessageBus, ESPMode::ThreadSafe> WeakBus)
		{
			AddDebuggerTab(WeakBus.Pin());
		});

		BusShutdownHandle = MessagingModule.OnMessageBusShutdown().AddLambda([this](TWeakPtr<IMessageBus, ESPMode::ThreadSafe> WeakBus)
		{
			RemoveDebuggerTab(WeakBus.Pin());
		});
	}

	void RemoveMessagingHooks(IMessagingModule& MessagingModule)
	{
		if (BusStartupHandle.IsValid())
		{
			MessagingModule.OnMessageBusStartup().Remove(BusStartupHandle);
		}

		if (BusShutdownHandle.IsValid())
		{
			MessagingModule.OnMessageBusShutdown().Remove(BusShutdownHandle);
		}
	}

	void AddDebuggerTab(TSharedPtr<IMessageBus, ESPMode::ThreadSafe> Bus)
	{
		if (!TabManager)
		{
			// Create the tab manager for the multiple message bus tabs
			TabManager = FGlobalTabmanager::Get()->NewTabManager(MajorDebuggerTab.ToSharedRef());
			const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("MessagingDebuggerMajorTabLayout_v1.0")
				->AddArea
				(
					FTabManager::NewPrimaryArea()
					->SetOrientation(Orient_Horizontal)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(MessagingDebuggerDocumentTabName, ETabState::ClosedTab)
					)
				);
			MajorDebuggerTab->SetContent(TabManager->RestoreFrom(Layout, MajorDebuggerTab->GetParentWindow()).ToSharedRef());
		}

		FText TabName = FText::FormatOrdered(LOCTEXT("OtherTabTitle", "{0}"), FText::FromString(Bus->GetName()));
		TSharedRef<SDockTab> Tab = SNew(SDockTab)
			.TabRole(ETabRole::PanelTab)
			.Label(TabName)
			// prevent tabs from being closed while still the associated bus still exists
			.OnCanCloseTab(SDockTab::FCanCloseTab::CreateLambda([this, WeakBus = TWeakPtr<IMessageBus, ESPMode::ThreadSafe>(Bus)]()
			{
				return !DebuggerTabs.Contains(WeakBus);
			}));

		TSharedPtr<SWidget> TabContent = SNew(SMessagingDebugger, Tab, MajorDebuggerTab->GetParentWindow(), Bus->GetTracer(), Style.ToSharedRef());
		Tab->SetContent(TabContent.ToSharedRef());
		DebuggerTabs.Add(Bus, Tab);

		TabManager->InsertNewDocumentTab(MessagingDebuggerDocumentTabName, FTabManager::ESearchPreference::Type::RequireClosedTab, Tab);
	}

	void RemoveDebuggerTab(TSharedPtr<IMessageBus, ESPMode::ThreadSafe> Bus)
	{
		// Close the tab associated with the bus that is shutting down.
		TSharedPtr<SDockTab> Tab;
		DebuggerTabs.RemoveAndCopyValue(Bus, Tab);
		if (Tab)
		{
			Tab->RequestCloseTab();
		}
	}

	void OnDebuggerTabClosed(TSharedRef<SDockTab>)
	{
		// Remove messaging hooks and release reference when closing the major tab.
		RemoveMessagingHooks(IMessagingModule::Get());
		MajorDebuggerTab.Reset();
		TabManager.Reset();
	}

private:

	/** The plug-ins style set. */
	TSharedPtr<ISlateStyle> Style;

	/** Major tab for holding message bus tabs. */
	TSharedPtr<SDockTab> MajorDebuggerTab;

	/** Holds the tab manager that manages multiple bus tabs. */
	TSharedPtr<FTabManager> TabManager;

	/** Map of opened MessageBus to their tab. */
	TMap<TWeakPtr<IMessageBus, ESPMode::ThreadSafe>, TSharedPtr<SDockTab>> DebuggerTabs;

	/** Handle to add tab as new message bus are started. */
	FDelegateHandle BusStartupHandle;

	/** Handle to remove tab as message bus are shutdown. */
	FDelegateHandle BusShutdownHandle;
};

IMPLEMENT_MODULE(FMessagingDebuggerModule, MessagingDebugger);


#undef LOCTEXT_NAMESPACE
