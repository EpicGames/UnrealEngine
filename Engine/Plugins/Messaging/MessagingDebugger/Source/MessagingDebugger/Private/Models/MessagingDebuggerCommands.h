// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"


#define LOCTEXT_NAMESPACE "FMessagingDebuggerCommands"


class FMessagingDebuggerCommands
	: public TCommands<FMessagingDebuggerCommands>
{
public:

	/** Default constructor. */
	FMessagingDebuggerCommands()
		: TCommands<FMessagingDebuggerCommands>("MessagingDebugger", NSLOCTEXT("Contexts", "MessagingDebugger", "Messaging Debugger"), NAME_None, "MessagingDebuggerStyle")
	{ }

public:

	//~ TCommands interface

	virtual void RegisterCommands() override
	{
		UI_COMMAND(BreakDebugger, "Break", "Break the debugger at the next message", EUserInterfaceActionType::Button, FInputChord(EKeys::Pause, /*EModifierKey::Control |*/ EModifierKey::Alt));
		UI_COMMAND(ClearHistory, "Clear History", "Clears the message history list", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ContinueDebugger, "Continue", "Continues debugging", EUserInterfaceActionType::Button, FInputChord(EKeys::F5));
		UI_COMMAND(StartDebugger, "Start", "Start the debugger", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(StepDebugger, "Step", "Step over the current message", EUserInterfaceActionType::Button, FInputChord(EKeys::F10));
		UI_COMMAND(StopDebugger, "Stop", "Stop the debugger", EUserInterfaceActionType::Button, FInputChord(EKeys::F5, EModifierKey::Shift));
	}

public:

	/** Holds a command that breaks at the next message. */
	TSharedPtr<FUICommandInfo> BreakDebugger;

	/** Holds a command that clears the message history. */
	TSharedPtr<FUICommandInfo> ClearHistory;

	/** Holds a command that continues debugging. */
	TSharedPtr<FUICommandInfo> ContinueDebugger;

	/** Holds a command that starts debugging. */
	TSharedPtr<FUICommandInfo> StartDebugger;

	/** Holds a command that continues debugging for one step. */
	TSharedPtr<FUICommandInfo> StepDebugger;

	/** Holds a command that stops debugging. */
	TSharedPtr<FUICommandInfo> StopDebugger;
};


#undef LOCTEXT_NAMESPACE
