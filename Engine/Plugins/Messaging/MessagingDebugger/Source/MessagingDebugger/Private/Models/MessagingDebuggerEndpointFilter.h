// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageTracer.h"


/**
 * Implements a filter for the message endpoints list.
 */
class FMessagingDebuggerEndpointFilter
{
public:

	/**
	 * Filters the specified endpoint based on the current filter settings.
	 *
	 * @param EndpointInfo The endpoint to filter.
	 * @return true if the endpoint passed the filter, false otherwise.
	 */
	bool FilterEndpoint(const TSharedPtr<FMessageTracerEndpointInfo>& EndpointInfo) const
	{
		if (!EndpointInfo.IsValid())
		{
			return false;
		}

		// filter by search string
		TArray<FString> FilterSubstrings;
		
		if (FilterString.ParseIntoArray(FilterSubstrings, TEXT(" "), true) > 0)
		{
			for (int32 SubstringIndex = 0; SubstringIndex < FilterSubstrings.Num(); ++SubstringIndex)
			{
				if (!EndpointInfo->Name.ToString().Contains(FilterSubstrings[SubstringIndex]))
				{
					return false;
				}
			}
		}

		return true;
	}

	/**
	 * Sets the filter string.
	 *
	 * @param InFilterString The filter string to set.
	 */
	void SetFilterString(const FString& InFilterString)
	{
		FilterString = InFilterString;
		ChangedEvent.Broadcast();
	}

public:

	/**
	 * Gets an event delegate that is invoked when the filter settings changed.
	 *
	 * @return The event delegate.
	 */
	DECLARE_EVENT(FMessagingDebuggerEndpointFilter, FOnMessagingEndpointFilterChanged);
	FOnMessagingEndpointFilterChanged& OnChanged()
	{
		return ChangedEvent;
	}

private:

	/** Holds the filter string used to filter endpoints by their names. */
	FString FilterString;

private:

	/** Holds an event delegate that is invoked when the filter settings changed. */
	FOnMessagingEndpointFilterChanged ChangedEvent;
};
