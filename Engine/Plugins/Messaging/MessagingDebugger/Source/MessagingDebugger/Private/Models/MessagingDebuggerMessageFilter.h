// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

struct FMessageTracerMessageInfo;


/**
 * Implements a filter for the message history list.
 */
class FMessagingDebuggerMessageFilter
{
public:

	/**
	 * Filters the specified message based on the current filter settings.
	 *
	 * @param MessageInfo The message to filter.
	 * @return true if the message passed the filter, false otherwise.
	 */
	bool FilterEndpoint(const TSharedPtr<FMessageTracerMessageInfo>& MessageInfo) const
	{
		if (!MessageInfo.IsValid())
		{
			return false;
		}

		// @todo gmp: implement message filters

		return true;
	}

public:

	/**
	 * Gets an event delegate that is invoked when the filter settings changed.
	 *
	 * @return The event delegate.
	 */
	DECLARE_EVENT(FMessagingDebuggerMessageFilter, FOnMessagingMessageFilterChanged);
	FOnMessagingMessageFilterChanged& OnChanged()
	{
		return ChangedEvent;
	}

private:

	/** Holds an event delegate that is invoked when the filter settings changed. */
	FOnMessagingMessageFilterChanged ChangedEvent;
};
