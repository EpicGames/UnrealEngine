// Copyright Epic Games, Inc. All Rights Reserved.

#include "Models/MessagingDebuggerModel.h"

#include "IMessageTracer.h"


/* FMessagingDebuggerModel interface
 *****************************************************************************/

void FMessagingDebuggerModel::ClearVisibilities()
{
	InvisibleEndpoints.Empty();
	InvisibleTypes.Empty();

	MessageVisibilityChangedEvent.Broadcast();
}


TSharedPtr<FMessageTracerEndpointInfo> FMessagingDebuggerModel::GetSelectedEndpoint() const
{
	return SelectedEndpoint;
}


TSharedPtr<FMessageTracerMessageInfo> FMessagingDebuggerModel::GetSelectedMessage() const
{
	return SelectedMessage;
}


bool FMessagingDebuggerModel::IsEndpointVisible(const TSharedRef<FMessageTracerEndpointInfo>& EndpointInfo) const
{
	return !InvisibleEndpoints.Contains(EndpointInfo);
}


bool FMessagingDebuggerModel::IsMessageVisible(const TSharedRef<FMessageTracerMessageInfo>& MessageInfo) const
{
	if (!IsEndpointVisible(MessageInfo->SenderInfo.ToSharedRef()) || !IsTypeVisible(MessageInfo->TypeInfo.ToSharedRef()))
	{
		return false;
	}

	if (MessageInfo->DispatchStates.Num() > 0)
	{
		for (const auto& DispatchStatePair : MessageInfo->DispatchStates)
		{
			const TSharedPtr<FMessageTracerDispatchState>& DispatchState = DispatchStatePair.Value;

			if (IsEndpointVisible(DispatchState->EndpointInfo.ToSharedRef()))
			{
				return true;
			}
		}
	}

	return true;
}


bool FMessagingDebuggerModel::IsTypeVisible(const TSharedRef<FMessageTracerTypeInfo>& TypeInfo) const
{
	return !InvisibleTypes.Contains(TypeInfo);
}


void FMessagingDebuggerModel::SelectEndpoint(const TSharedPtr<FMessageTracerEndpointInfo>& EndpointInfo)
{
	if (SelectedEndpoint != EndpointInfo)
	{
		SelectedEndpoint = EndpointInfo;
		SelectedEndpointChangedEvent.Broadcast();
	}
}


void FMessagingDebuggerModel::SelectMessage(const TSharedPtr<FMessageTracerMessageInfo>& MessageInfo)
{
	if (SelectedMessage != MessageInfo)
	{
		SelectedMessage = MessageInfo;
		SelectedMessageChangedEvent.Broadcast();
	}
}


void FMessagingDebuggerModel::SetEndpointVisibility(const TSharedRef<FMessageTracerEndpointInfo>& EndpointInfo, bool Visible)
{
	if (Visible)
	{
		InvisibleEndpoints.Remove(EndpointInfo);
	}
	else
	{
		InvisibleEndpoints.AddUnique(EndpointInfo);
	}

	MessageVisibilityChangedEvent.Broadcast();
}


void FMessagingDebuggerModel::SetTypeVisibility(const TSharedRef<FMessageTracerTypeInfo>& TypeInfo, bool Visible)
{
	if (Visible)
	{
		InvisibleTypes.Remove(TypeInfo);
	}
	else
	{
		InvisibleTypes.AddUnique(TypeInfo);
	}

	MessageVisibilityChangedEvent.Broadcast();
}
