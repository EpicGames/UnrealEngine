// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

struct FMessageTracerEndpointInfo;
struct FMessageTracerMessageInfo;
struct FMessageTracerTypeInfo;


/**
 * Implements a view model for the messaging debugger.
 */
class FMessagingDebuggerModel
{
public:

	/**
	 * Clears all visibility filters (to show all messages).
	 */
	void ClearVisibilities();

	/**
	 * Gets the endpoint that is currently selected in the endpoint list.
	 *
	 * @return The selected endpoint.
	 */
	TSharedPtr<FMessageTracerEndpointInfo> GetSelectedEndpoint() const;

	/**
	 * Gets the message that is currently selected in the message history.
	 *
	 * @return The selected message.
	 */
	TSharedPtr<FMessageTracerMessageInfo> GetSelectedMessage() const;

	/** 
	 * Checks whether messages of the given message endpoint should be visible.
	 *
	 * @param Endpoint The information of the endpoint to check.
	 * @return true if the endpoint's messages should be visible, false otherwise.
	 */
	bool IsEndpointVisible(const TSharedRef<FMessageTracerEndpointInfo>& EndpointInfo) const;

	/**
	 * Checks whether the given message should be visible.
	 *
	 * @param MessageInfo The information of the message to check.
	 * @return true if the message should be visible, false otherwise.
	 */
	bool IsMessageVisible(const TSharedRef<FMessageTracerMessageInfo>& MessageInfo) const;

	/**
	 * Checks whether messages of the given type should be visible.
	 *
	 * @param TypeInfo The information of the message type to check.
	 * @return true if messages of the given type should be visible, false otherwise.
	 */
	bool IsTypeVisible(const TSharedRef<FMessageTracerTypeInfo>& TypeInfo) const;

	/**
	 * Selects the specified endpoint (or none if nullptr).
	 *
	 * @param EndpointInfo The information of the endpoint to select.
	 */
	void SelectEndpoint(const TSharedPtr<FMessageTracerEndpointInfo>& EndpointInfo);

	/**
	 * Selects the specified message (or none if nullptr).
	 *
	 * @param MessageInfo The information of the message to select.
	 */
	void SelectMessage(const TSharedPtr<FMessageTracerMessageInfo>& MessageInfo);

	/**
	 * Sets whether messages for the given endpoint should be visible.
	 *
	 * @param Endpoint The information for the endpoint.
	 * @param Visible Whether the endpoint's messages should be visible or not.
	 */
	void SetEndpointVisibility(const TSharedRef<FMessageTracerEndpointInfo>& EndpointInfo, bool Visible);

	/**
	 * Sets whether messages for the given message type should be visible.
	 *
	 * @param Endpoint The information for the message type.
	 * @param Visible Whether messages of the given type should be visible or not.
	 */
	void SetTypeVisibility(const TSharedRef<FMessageTracerTypeInfo>& TypeInfo, bool Visible);

public:

	/**
	 * Gets an event delegate that is invoked when the filter settings changed.
	 *
	 * @return The event delegate.
	 */
	DECLARE_EVENT(FMessagingDebuggerModel, FOnMessageVisibilityChanged);
	FOnMessageVisibilityChanged& OnMessageVisibilityChanged()
	{
		return MessageVisibilityChangedEvent;
	}

	/**
	 * Gets an event delegate that is invoked when the selected endpoint has changed.
	 *
	 * @return The event delegate.
	 */
	DECLARE_EVENT(FMessagingDebuggerModel, FOnSelectedEndpointChanged);
	FOnSelectedEndpointChanged& OnSelectedEndpointChanged()
	{
		return SelectedEndpointChangedEvent;
	}

	/**
	 * Gets an event delegate that is invoked when the selected message has changed.
	 *
	 * @return The event delegate.
	 */
	DECLARE_EVENT(FMessagingDebuggerModel, FOnSelectedMessageChanged);
	FOnSelectedMessageChanged& OnSelectedMessageChanged()
	{
		return SelectedMessageChangedEvent;
	}

private:

	/** Holds the collection of invisible message endpoints. */
	TArray<TSharedPtr<FMessageTracerEndpointInfo>> InvisibleEndpoints;

	/** Holds the collection of invisible message types. */
	TArray<TSharedPtr<FMessageTracerTypeInfo>> InvisibleTypes;

	/** Holds a pointer to the sole endpoint that is currently selected in the endpoint list. */
	TSharedPtr<FMessageTracerEndpointInfo> SelectedEndpoint;

	/** Holds a pointer to the message that is currently selected in the message history. */
	TSharedPtr<FMessageTracerMessageInfo> SelectedMessage;

private:

	/** Holds an event delegate that is invoked when the visibility of messages has changed. */
	FOnMessageVisibilityChanged MessageVisibilityChangedEvent;

	/** Holds an event delegate that is invoked when the selected endpoint has changed. */
	FOnSelectedEndpointChanged SelectedEndpointChangedEvent;

	/** Holds an event delegate that is invoked when the selected message has changed. */
	FOnSelectedMessageChanged SelectedMessageChangedEvent;
};
