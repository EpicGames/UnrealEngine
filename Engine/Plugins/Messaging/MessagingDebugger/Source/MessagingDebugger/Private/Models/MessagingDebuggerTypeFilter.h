// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageTracer.h"


/**
 * Implements a filter for the message endpoints list.
 */
class FMessagingDebuggerTypeFilter
{
public:
	
	/**
	 * Filters the specified message type based on the current filter settings.
	 *
	 * @param TypeInfo The message type to filter.
	 * @return true if the endpoint passed the filter, false otherwise.
	 */
	bool FilterType(const TSharedPtr<FMessageTracerTypeInfo>& TypeInfo) const
	{
		if (!TypeInfo.IsValid())
		{
			return false;
		}

		// filter by search string
		TArray<FString> FilterSubstrings;
		
		if (FilterString.ParseIntoArray(FilterSubstrings, TEXT(" "), true) > 0)
		{
			FString TypeName = TypeInfo->TypePathName.ToString();

			for (int32 SubstringIndex = 0; SubstringIndex < FilterSubstrings.Num(); ++SubstringIndex)
			{
				if (!TypeName.Contains(FilterSubstrings[SubstringIndex]))
				{
					return false;
				}
			}
		}

		return true;
	}

	/**
	 * Sets the filter string.
	 *
	 * @param InFilterString The filter string to set.
	 */
	void SetFilterString(const FString& InFilterString)
	{
		FilterString = InFilterString;
		ChangedEvent.Broadcast();
	}

public:

	/**
	 * Gets an event delegate that is invoked when the filter settings changed.
	 *
	 * @return The event delegate.
	 */
	DECLARE_EVENT(FMessagingDebuggerTypeFilter, FOnMessagingEndpointFilterChanged);
	FOnMessagingEndpointFilterChanged& OnChanged()
	{
		return ChangedEvent;
	}

private:

	/** Holds the filter string used to filter message types by their names. */
	FString FilterString;

private:

	/** Holds an event delegate that is invoked when the filter settings changed. */
	FOnMessagingEndpointFilterChanged ChangedEvent;
};
