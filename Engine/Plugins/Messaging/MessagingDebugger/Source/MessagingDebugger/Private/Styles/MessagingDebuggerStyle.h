// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateTypes.h"


#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define BOX_BRUSH(RelativePath, ...) FSlateBoxBrush(RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define BORDER_BRUSH(RelativePath, ...) FSlateBorderBrush(RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)


/**
 * Implements the visual style of the messaging debugger UI.
 */
class FMessagingDebuggerStyle final
	: public FSlateStyleSet
{
public:

	/** Default constructor. */
	 FMessagingDebuggerStyle()
		 : FSlateStyleSet("MessagingDebuggerStyle")
	 {
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);

		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Messaging/MessagingDebugger/Content"));

		// general
		Set("BreakpointBorder", new BOX_BRUSH("GroupBorder", FMargin(4.0f / 16.0f)));
		Set("GroupBorder", new BOX_BRUSH("GroupBorder", FMargin(4.0f / 16.0f)));

		// tab icons
		Set("BreakpointsTabIcon", new IMAGE_BRUSH("icon_tab_Breakpoints_16x", Icon16x16));
		Set("EndpointDetailsTabIcon", new IMAGE_BRUSH("icon_tab_EndpointDetails_16x", Icon16x16));
		Set("EndpointsTabIcon", new IMAGE_BRUSH("icon_tab_Endpoints_16x", Icon16x16));
		Set("InteractionGraphTabIcon", new IMAGE_BRUSH("icon_tab_InteractionGraph_16x", Icon16x16));
		Set("InterceptorsTabIcon", new IMAGE_BRUSH("icon_tab_Interceptors_16x", Icon16x16));
		Set("MessageDataTabIcon", new IMAGE_BRUSH("icon_tab_MessageData_16x", Icon16x16));
		Set("MessageDetailsTabIcon", new IMAGE_BRUSH("icon_tab_MessageDetails_16x", Icon16x16));
		Set("MessageHistoryTabIcon", new IMAGE_BRUSH("icon_tab_MessageHistory_16x", Icon16x16));
		Set("MessageTypesTabIcon", new IMAGE_BRUSH("icon_tab_MessageTypes_16x", Icon16x16));
		Set("MessagingDebuggerTabIcon", new IMAGE_BRUSH("icon_tab_MessagingDebugger_16x", Icon16x16));
		Set("ToolbarTabIcon", new IMAGE_BRUSH("icon_tab_Toolbar_16x", Icon16x16));

		// toolbar icons
		Set("MessagingDebugger.BreakDebugger", new IMAGE_BRUSH("icon_debugger_break_40x", Icon40x40));
		Set("MessagingDebugger.BreakDebugger.Small", new IMAGE_BRUSH("icon_debugger_break_40x", Icon20x20));
		Set("MessagingDebugger.ClearHistory", new IMAGE_BRUSH("icon_history_clear_40x", Icon40x40));
		Set("MessagingDebugger.ClearHistory.Small", new IMAGE_BRUSH("icon_history_clear_40x", Icon20x20));
		Set("MessagingDebugger.ContinueDebugger", new IMAGE_BRUSH("icon_debugger_continue_40x", Icon40x40));
		Set("MessagingDebugger.ContinueDebugger.Small", new IMAGE_BRUSH("icon_debugger_continue_40x", Icon20x20));
		Set("MessagingDebugger.StartDebugger", new IMAGE_BRUSH("icon_debugger_start_40x", Icon40x40));
		Set("MessagingDebugger.StartDebugger.Small", new IMAGE_BRUSH("icon_debugger_start_40x", Icon20x20));
		Set("MessagingDebugger.StepDebugger", new IMAGE_BRUSH("icon_debugger_step_40x", Icon40x40));
		Set("MessagingDebugger.StepDebugger.Small", new IMAGE_BRUSH("icon_debugger_step_40x", Icon20x20));
		Set("MessagingDebugger.StopDebugger", new IMAGE_BRUSH("icon_debugger_stop_40x", Icon40x40));
		Set("MessagingDebugger.StopDebugger.Small", new IMAGE_BRUSH("icon_debugger_stop_40x", Icon20x20));

		// misc icons
		Set("Break", new IMAGE_BRUSH("icon_break_16x", Icon16x16));
		Set("BreakColumn", new IMAGE_BRUSH("BreakpointBorder", Icon16x16));
		Set("BreakDisabled", new IMAGE_BRUSH("icon_break_disabled_16x", Icon16x16));
		Set("BreakIn", new IMAGE_BRUSH("icon_break_in_16x", Icon16x16));
		Set("BreakOut", new IMAGE_BRUSH("icon_break_out_16x", Icon16x16));
		Set("BreakInOut", new IMAGE_BRUSH("icon_break_inout_16x", Icon16x16));
		Set("DispatchDirect", new IMAGE_BRUSH("icon_dispatch_direct_16x", Icon16x16));
		Set("DispatchPending", new IMAGE_BRUSH("icon_dispatch_pending_16x", Icon16x16));
		Set("DispatchTaskGraph", new IMAGE_BRUSH("icon_dispatch_taskgraph_16x", Icon16x16));
		Set("DeadMessage", new IMAGE_BRUSH("icon_message_dead_16x", Icon16x16));
		Set("ForwardedMessage", new IMAGE_BRUSH("icon_message_forwarded_16x", Icon16x16));
		Set("InboundMessage", new IMAGE_BRUSH("icon_message_inbound_16x", Icon16x16));
		Set("LocalEndpoint", new IMAGE_BRUSH("icon_endpoint_local_16x", Icon16x16));
		Set("OutboundMessage", new IMAGE_BRUSH("icon_message_outbound_16x", Icon16x16));
		Set("PublishedMessage", new IMAGE_BRUSH("icon_message_published_16x", Icon16x16));
		Set("RemoteEndpoint", new IMAGE_BRUSH("icon_endpoint_remote_16x", Icon16x16));
		Set("SentMessage", new IMAGE_BRUSH("icon_message_sent_16x", Icon16x16));
		Set("Visibility", new IMAGE_BRUSH("icon_visible_16x", Icon16x16));

		Set("VisibilityCheckbox", FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::CheckBox)
			.SetUncheckedImage(IMAGE_BRUSH("icon_invisible_16x", Icon16x16))
			.SetUncheckedHoveredImage(IMAGE_BRUSH("icon_invisible_16x", Icon16x16))
			.SetUncheckedPressedImage(IMAGE_BRUSH("icon_invisible_16x", Icon16x16, FLinearColor(0.5f, 0.5f, 0.5f)))
			.SetCheckedImage(IMAGE_BRUSH("icon_visible_16x", Icon16x16))
			.SetCheckedHoveredImage(IMAGE_BRUSH("icon_visible_16x", Icon16x16, FLinearColor(0.5f, 0.5f, 0.5f)))
			.SetCheckedPressedImage(IMAGE_BRUSH("icon_visible_16x", Icon16x16))
			.SetUndeterminedImage(IMAGE_BRUSH("icon_invisible_16x", Icon16x16))
			.SetUndeterminedHoveredImage(IMAGE_BRUSH("icon_invisible_16x", Icon16x16))
			.SetUndeterminedPressedImage(IMAGE_BRUSH("icon_invisible_16x", Icon16x16, FLinearColor(0.5f, 0.5f, 0.5f)))
		);
		
		FSlateStyleRegistry::RegisterSlateStyle(*this);
	 }

	 /** Virtual destructor. */
	 virtual ~FMessagingDebuggerStyle()
	 {
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	 }
};


#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef BORDER_BRUSH
