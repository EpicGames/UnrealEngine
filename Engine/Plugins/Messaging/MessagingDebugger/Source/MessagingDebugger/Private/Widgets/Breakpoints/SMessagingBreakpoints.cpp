// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Breakpoints/SMessagingBreakpoints.h"

#include "Framework/Views/TableViewMetadata.h"
#include "IMessageTracer.h"
#include "IMessageTracerBreakpoint.h"

#include "Widgets/Breakpoints/SMessagingBreakpointsTableRow.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SMessagingBreakpoints"


/* SMessagingBreakpoints interface
 *****************************************************************************/

void SMessagingBreakpoints::Construct(const FArguments& InArgs, const TSharedRef<ISlateStyle>& InStyle, const TSharedRef<IMessageTracer, ESPMode::ThreadSafe>& InTracer)
{
	typedef SListView<TSharedPtr<IMessageTracerBreakpoint, ESPMode::ThreadSafe>> IMessageTracerBreakPointPtr;

	Style = InStyle;
	Tracer = InTracer;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f, 4.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
					.BorderImage(InStyle->GetBrush("GroupBorder"))
					.Padding(0.0f)
					[
						// message list
						SAssignNew(BreakpointListView, IMessageTracerBreakPointPtr)
							.ListItemsSource(&BreakpointList)
							.SelectionMode(ESelectionMode::Multi)
							.OnGenerateRow(this, &SMessagingBreakpoints::HandleBreakpointListGenerateRow)
							.OnSelectionChanged(this, &SMessagingBreakpoints::HandleBreakpointListSelectionChanged)
							.HeaderRow
							(
								SNew(SHeaderRow)

								+ SHeaderRow::Column("Name")
									.DefaultLabel(LOCTEXT("BreakpointListNameColumnHeader", "Name"))
									.FillWidth(1.0f)

								+ SHeaderRow::Column("HitCount")
									.DefaultLabel(LOCTEXT("BreakpointListHitCountColumnHeader", "Hit Count"))
									.FixedWidth(64.0f)
							)
					]
			]
	];
}


/* SMessagingBreakpoints callbacks
 *****************************************************************************/

TSharedRef<ITableRow> SMessagingBreakpoints::HandleBreakpointListGenerateRow(TSharedPtr<IMessageTracerBreakpoint, ESPMode::ThreadSafe> Breakpoint, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SMessagingBreakpointsTableRow, OwnerTable, Breakpoint.ToSharedRef())
		.Style(Style);
}


void SMessagingBreakpoints::HandleBreakpointListSelectionChanged(TSharedPtr<IMessageTracerBreakpoint, ESPMode::ThreadSafe> InItem, ESelectInfo::Type SelectInfo)
{

}


#undef LOCTEXT_NAMESPACE
