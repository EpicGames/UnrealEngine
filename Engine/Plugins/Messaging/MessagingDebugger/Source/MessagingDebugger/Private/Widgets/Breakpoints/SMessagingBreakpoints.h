// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ITableRow;
namespace ESelectInfo { enum Type : int; }
template <typename ItemType> class SListView;

class IMessageTracer;
class IMessageTracerBreakpoint;
class ISlateStyle;
class STableViewBase;


/**
 * Implements the message breakpoints panel.
 */
class SMessagingBreakpoints
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessagingBreakpoints) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InStyle The visual style to use for this widget.
	 * @param InTracer The message tracer.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<ISlateStyle>& InStyle, const TSharedRef<IMessageTracer, ESPMode::ThreadSafe>& InTracer);

private:

	/** Handles generating a row widget for the endpoint list view. */
	TSharedRef<ITableRow> HandleBreakpointListGenerateRow(TSharedPtr<IMessageTracerBreakpoint, ESPMode::ThreadSafe> EndpointInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/** Handles the selection of endpoints. */
	void HandleBreakpointListSelectionChanged(TSharedPtr<IMessageTracerBreakpoint, ESPMode::ThreadSafe> InItem, ESelectInfo::Type SelectInfo);

private:

	/** Holds the filtered list of historic messages. */
	TArray<TSharedPtr<IMessageTracerBreakpoint, ESPMode::ThreadSafe>> BreakpointList;

	/** Holds the message list view. */
	TSharedPtr<SListView<TSharedPtr<IMessageTracerBreakpoint, ESPMode::ThreadSafe>>> BreakpointListView;

	/** Holds the widget's visual style. */
	TSharedPtr<ISlateStyle> Style;

	/** Holds a pointer to the message bus tracer. */
	TSharedPtr<IMessageTracer, ESPMode::ThreadSafe> Tracer;
};
