// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Breakpoints/SMessagingBreakpointsTableRow.h"

#include "IMessageTracerBreakpoint.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SMessagingBreakpointsTableRow"


/* SMessagingBreakpointsTableRow interface
 *****************************************************************************/

void SMessagingBreakpointsTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<IMessageTracerBreakpoint, ESPMode::ThreadSafe> InBreakpoint)
{
	check(InArgs._Style.IsValid());

	Breakpoint = InBreakpoint;
	Style = InArgs._Style;

	SMultiColumnTableRow<TSharedPtr<IMessageTracerBreakpoint, ESPMode::ThreadSafe>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}


/* SMultiColumnTableRow interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SMessagingBreakpointsTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == "HitCount")
	{
		return SNullWidget::NullWidget;
	}
	else if (ColumnName == "Name")
	{
		return SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				[
					SNew(SCheckBox)
						.IsChecked(ECheckBoxState::Checked)
				]

			+ SHorizontalBox::Slot()
				.Padding(FMargin(4.0f, 0.0f))
				[
					SNew(SImage)
						.Image(Style->GetBrush("BreakDisabled"))
				]

			+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("TempNameColumn", "@todo"))
				];
	}

	return SNullWidget::NullWidget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE

