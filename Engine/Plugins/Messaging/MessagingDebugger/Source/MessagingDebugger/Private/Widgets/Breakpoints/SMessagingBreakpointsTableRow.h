// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class IMessageTracerBreakpoint;
class ISlateStyle;
class STableViewBase;


/**
 * Implements a row widget for the session console log.
 */
class SMessagingBreakpointsTableRow
	: public SMultiColumnTableRow<TSharedPtr<IMessageTracerBreakpoint, ESPMode::ThreadSafe>>
{
public:

	SLATE_BEGIN_ARGS(SMessagingBreakpointsTableRow) { }
		SLATE_ARGUMENT(TSharedPtr<ISlateStyle>, Style)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InOwnerTableView The table view that owns this row.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<IMessageTracerBreakpoint, ESPMode::ThreadSafe> InBreakpoint);

public:

	//~ SMultiColumnTableRow interface

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	/** Holds a pointer to the breakpoint that is shown in this row. */
	TSharedPtr<IMessageTracerBreakpoint, ESPMode::ThreadSafe> Breakpoint;

	/** Holds the widget's visual style. */
	TSharedPtr<ISlateStyle> Style;
};
