// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/EndpointDetails/SMessagingAddressTableRow.h"

#include "IMessageTracer.h"
#include "SlateOptMacros.h"

#include "Models/MessagingDebuggerModel.h"
#include "Widgets/Views/SListView.h"


namespace MessagingAddressTableRow
{
	static const FNumberFormattingOptions TimeRegisteredFormattingOptions = FNumberFormattingOptions()
		.SetMinimumFractionalDigits(5)
		.SetMaximumFractionalDigits(5);
}


#define LOCTEXT_NAMESPACE "SMessagingAddressTableRow"


/* SMessagingAddressTableRow interface
 *****************************************************************************/

void SMessagingAddressTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FMessagingDebuggerModel>& InModel)
{
	check(InArgs._Style.IsValid());
	check(InArgs._AddressInfo.IsValid());

	AddressInfo = InArgs._AddressInfo;
	Model = InModel;
	Style = InArgs._Style;

	SMultiColumnTableRow<TSharedPtr<FMessageTracerAddressInfo>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}


/* SMultiColumnTableRow interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SMessagingAddressTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == "Address")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(FText::FromString(AddressInfo->Address.ToString()))
			];
	}
	else if (ColumnName == "TimeRegistered")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(FText::AsNumber(AddressInfo->TimeRegistered - GStartTime, &MessagingAddressTableRow::TimeRegisteredFormattingOptions))
			];
	}
	else if (ColumnName == "TimeUnregistered")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(this, &SMessagingAddressTableRow::HandleTimeUnregisteredText)
			];
	}

	return SNullWidget::NullWidget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SMessagingAddressTableRow callbacks
 *****************************************************************************/

FText SMessagingAddressTableRow::HandleTimeUnregisteredText() const
{
	if (AddressInfo->TimeUnregistered > 0.0)
	{
		return FText::AsNumber(AddressInfo->TimeUnregistered - GStartTime, &MessagingAddressTableRow::TimeRegisteredFormattingOptions);
	}

	return LOCTEXT("Never", "Never");
}


#undef LOCTEXT_NAMESPACE
