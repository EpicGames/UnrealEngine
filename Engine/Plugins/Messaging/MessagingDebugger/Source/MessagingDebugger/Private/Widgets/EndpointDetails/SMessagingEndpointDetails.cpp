// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/EndpointDetails/SMessagingEndpointDetails.h"

#include "Framework/Views/TableViewMetadata.h"
#include "IMessageTracer.h"
#include "Widgets/Layout/SGridPanel.h"

#include "Models/MessagingDebuggerModel.h"
#include "Widgets/EndpointDetails/SMessagingAddressTableRow.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SMessagingEndpointDetails"


/* SMessagingEndpointDetails interface
 *****************************************************************************/

void SMessagingEndpointDetails::Construct(const FArguments& InArgs, const TSharedRef<FMessagingDebuggerModel>& InModel, const TSharedRef<ISlateStyle>& InStyle)
{
	Model = InModel;
	Style = InStyle;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.0f, 2.0f)
			[
				SNew(SGridPanel)
					.FillColumn(1, 1.0f)

				// received messages count
				+ SGridPanel::Slot(0, 0)
					.Padding(0.0f, 4.0f, 32.0f, 4.0f)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("EndpointDetailsReceivedMessagesLabel", "Messages Received:"))
					]

				+ SGridPanel::Slot(1, 0)
					.Padding(0.0f, 4.0f)
					[
						SNew(STextBlock)
							.Text(this, &SMessagingEndpointDetails::HandleEndpointDetailsReceivedMessagesText)
					]

				// sent messages count
				+ SGridPanel::Slot(0, 1)
					.Padding(0.0f, 4.0f, 32.0f, 4.0f)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("EndpointDetailsReceivedLabel", "Messages Sent:"))
					]

				+ SGridPanel::Slot(1, 1)
					.Padding(0.0f, 4.0f)
					[
						SNew(STextBlock)
							.Text(this, &SMessagingEndpointDetails::HandleEndpointDetailsSentMessagesText)
					]
			]

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f, 8.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
					.BorderImage(InStyle->GetBrush("GroupBorder"))
					.Padding(0.0f)
					[
						// address list
						SAssignNew(AddressListView, SListView<TSharedPtr<FMessageTracerAddressInfo>>)
							.ListItemsSource(&AddressList)
							.SelectionMode(ESelectionMode::None)
							.OnGenerateRow(this, &SMessagingEndpointDetails::HandleAddressListGenerateRow)
							.HeaderRow
							(
								SNew(SHeaderRow)

								+ SHeaderRow::Column("Address")
									.DefaultLabel(FText::FromString(TEXT("Addresses")))
									.FillWidth(1.0f)

								+ SHeaderRow::Column("TimeRegistered")
									.DefaultLabel(LOCTEXT("AddressListTimeRegisteredColumnHeader", "Time Registered"))
									.FixedWidth(112.0f)
									.HAlignCell(HAlign_Right)
									.HAlignHeader(HAlign_Right)

								+ SHeaderRow::Column("TimeUnregistered")
									.DefaultLabel(LOCTEXT("AddressListTimeUnregisteredColumnHeader", "Time Unregistered"))
									.FixedWidth(112.0f)
									.HAlignCell(HAlign_Right)
									.HAlignHeader(HAlign_Right)
							)
					]
			]
	];
}


/* SCompoundWidget overrides
 *****************************************************************************/

void SMessagingEndpointDetails::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	RefreshAddressInfo();
}


/* SMessagingMessageDetails implementation
 *****************************************************************************/

void SMessagingEndpointDetails::RefreshAddressInfo()
{
	TSharedPtr<FMessageTracerEndpointInfo> SelectedEndpoint = Model->GetSelectedEndpoint();

	if (SelectedEndpoint.IsValid())
	{
		SelectedEndpoint->AddressInfos.GenerateValueArray(AddressList);
	}
	else
	{
		AddressList.Reset();
	}

	AddressListView->RequestListRefresh();
}


/* SMessagingEndpointDetails event handlers
 *****************************************************************************/

TSharedRef<ITableRow> SMessagingEndpointDetails::HandleAddressListGenerateRow(TSharedPtr<FMessageTracerAddressInfo> AddressInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SMessagingAddressTableRow, OwnerTable, Model.ToSharedRef())
		.AddressInfo(AddressInfo)
		.Style(Style);
}


FText SMessagingEndpointDetails::HandleEndpointDetailsReceivedMessagesText() const
{
	TSharedPtr<FMessageTracerEndpointInfo> SelectedEndpoint = Model->GetSelectedEndpoint();

	if (SelectedEndpoint.IsValid())
	{
		return FText::AsNumber(SelectedEndpoint->ReceivedMessages.Num());
	}

	return FText::GetEmpty();
}


FText SMessagingEndpointDetails::HandleEndpointDetailsSentMessagesText() const
{
	TSharedPtr<FMessageTracerEndpointInfo> SelectedEndpoint = Model->GetSelectedEndpoint();

	if (SelectedEndpoint.IsValid())
	{
		return FText::AsNumber(SelectedEndpoint->SentMessages.Num());
	}

	return FText::GetEmpty();
}


#undef LOCTEXT_NAMESPACE
