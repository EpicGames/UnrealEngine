// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ITableRow;
template <typename ItemType> class SListView;

class FMessagingDebuggerModel;
class ISlateStyle;
class STableViewBase;

struct FMessageTracerAddressInfo;


/**
* Implements the message types panel.
*/
class SMessagingEndpointDetails
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessagingEndpointDetails) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InModel The view model to use.
	 * @param InStyle The visual style to use for this widget.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FMessagingDebuggerModel>& InModel, const TSharedRef<ISlateStyle>& InStyle);

public:

	//~ SCompoundWidget overrides

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:

	/** Refreshes the endpoint's address information. */
	void RefreshAddressInfo();

private:

	/** Callback for generating a row widget for the address list view. */
	TSharedRef<ITableRow> HandleAddressListGenerateRow(TSharedPtr<FMessageTracerAddressInfo> AddressInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback for getting the number of received messages. */
	FText HandleEndpointDetailsReceivedMessagesText() const;

	/** Callback for getting the number of sent messages. */
	FText HandleEndpointDetailsSentMessagesText() const;

private:

	/** Holds the list of address information. */
	TArray<TSharedPtr<FMessageTracerAddressInfo>> AddressList;

	/** Holds the address information list view. */
	TSharedPtr<SListView<TSharedPtr<FMessageTracerAddressInfo>> > AddressListView;

	/** Holds a pointer to the view model. */
	TSharedPtr<FMessagingDebuggerModel> Model;

	/** Holds the widget's visual style. */
	TSharedPtr<ISlateStyle> Style;
};
