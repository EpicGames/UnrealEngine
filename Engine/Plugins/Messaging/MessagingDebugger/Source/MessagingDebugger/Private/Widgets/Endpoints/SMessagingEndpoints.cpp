// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Endpoints/SMessagingEndpoints.h"

#include "Framework/Views/TableViewMetadata.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SExpandableArea.h"

#include "Models/MessagingDebuggerEndpointFilter.h"
#include "Models/MessagingDebuggerModel.h"
#include "Widgets/Endpoints/SMessagingEndpointsFilterBar.h"
#include "Widgets/Endpoints/SMessagingEndpointsTableRow.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SMessagingEndpoints"


/* SMessagingEndpoints structors
*****************************************************************************/

SMessagingEndpoints::~SMessagingEndpoints()
{
	if (Model.IsValid())
	{
		Model->OnSelectedMessageChanged().RemoveAll(this);
	}
}


/* SMessagingEndpoints interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMessagingEndpoints::Construct(const FArguments& InArgs, const TSharedRef<FMessagingDebuggerModel>& InModel, const TSharedRef<ISlateStyle>& InStyle, const TSharedRef<IMessageTracer, ESPMode::ThreadSafe>& InTracer)
{
	Filter = MakeShareable(new FMessagingDebuggerEndpointFilter());
	Model = InModel;
	Style = InStyle;
	Tracer = InTracer;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SExpandableArea)
					.AreaTitle(LOCTEXT("EndpointFilterAreaTitle", "Endpoint Filter"))
					.InitiallyCollapsed(true)
					.Padding(8.0f)
					.BodyContent()
					[
						// filter bar
						SNew(SMessagingEndpointsFilterBar, Filter.ToSharedRef())
					]
			]

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f, 4.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
					.BorderImage(InStyle->GetBrush("GroupBorder"))
					.Padding(0.0f)
					[
						// message list
						SAssignNew(EndpointListView, SListView<TSharedPtr<FMessageTracerEndpointInfo>>)
							.ListItemsSource(&EndpointList)
							.SelectionMode(ESelectionMode::Multi)
							.OnGenerateRow(this, &SMessagingEndpoints::HandleEndpointListGenerateRow)
							.OnSelectionChanged(this, &SMessagingEndpoints::HandleEndpointListSelectionChanged)
							.HeaderRow
							(
								SNew(SHeaderRow)

								+ SHeaderRow::Column("Break")
									.FixedWidth(24.0f)
									.HAlignHeader(HAlign_Center)
									.VAlignHeader(VAlign_Center)
									.HeaderContent()
									[
										SNew(SImage)
											.Image(InStyle->GetBrush("BreakColumn"))
											.ToolTipText(LOCTEXT("EndpointListBreakpointsColumnTooltip", "Breakpoints"))
									]

								+ SHeaderRow::Column("Name")
									.DefaultLabel(LOCTEXT("EndpointListNameColumnHeader", "Name"))
									.FillWidth(0.9f)

								+ SHeaderRow::Column("Messages")
									.FillWidth(0.1f)
									.HAlignCell(HAlign_Right)
									.HAlignHeader(HAlign_Right)
									.VAlignHeader(VAlign_Center)
									.HeaderContent()
									[
										SNew(SImage)
											.Image(Style->GetBrush("SentMessage"))
											.ToolTipText(LOCTEXT("TypeListMessagesColumnTooltip", "Number of sent and received messages"))
									]

								+ SHeaderRow::Column("Visibility")
									.FixedWidth(26.0f)
									.HAlignCell(HAlign_Center)
									.HAlignHeader(HAlign_Center)
									.VAlignHeader(VAlign_Center)
									.HeaderContent()
									[
										SNew(SImage)
											.Image(InStyle->GetBrush("Visibility"))
											.ToolTipText(LOCTEXT("EndpointListVisibilityColumnTooltip", "Visibility"))
									]
							)
					]
			]
	];

	Filter->OnChanged().AddRaw(this, &SMessagingEndpoints::HandleFilterChanged);
	Model->OnSelectedMessageChanged().AddRaw(this, &SMessagingEndpoints::HandleModelSelectedMessageChanged);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SMessagingEndpoints::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// @todo gmp: fix this
	ReloadEndpointList();
}


/* SMessagingEndpoints implementation
 *****************************************************************************/

void SMessagingEndpoints::ReloadEndpointList()
{
	EndpointList.Reset();

	TArray<TSharedPtr<FMessageTracerEndpointInfo>> OutEndpoints;
	Tracer->GetEndpoints(OutEndpoints);

	for (int32 EndpointIndex = 0; EndpointIndex < OutEndpoints.Num(); ++EndpointIndex)
	{
		const TSharedPtr<FMessageTracerEndpointInfo>& Endpoint = OutEndpoints[EndpointIndex];

		if (Filter->FilterEndpoint(Endpoint))
		{
			EndpointList.Add(Endpoint);
		}
	}

	EndpointListView->RequestListRefresh();
}


/* SMessagingEndpoints callbacks
 *****************************************************************************/

TSharedRef<ITableRow> SMessagingEndpoints::HandleEndpointListGenerateRow(TSharedPtr<FMessageTracerEndpointInfo> EndpointInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SMessagingEndpointsTableRow, OwnerTable, Model.ToSharedRef())
		.EndpointInfo(EndpointInfo)
		.HighlightText(this, &SMessagingEndpoints::HandleEndpointListGetHighlightText)
		.Style(Style);
//		.ToolTipText(EndpointInfo->Address.ToString(EGuidFormats::DigitsWithHyphensInBraces));
}


FText SMessagingEndpoints::HandleEndpointListGetHighlightText() const
{
	return FText::GetEmpty();
	//return FilterBar->GetFilterText();
}


void SMessagingEndpoints::HandleEndpointListSelectionChanged(TSharedPtr<FMessageTracerEndpointInfo> InItem, ESelectInfo::Type SelectInfo)
{
	if (EndpointListView->GetSelectedItems().Num() == 1)
	{
		Model->SelectEndpoint(InItem);
	}
	else
	{
		Model->SelectEndpoint(nullptr);
	}
}


void SMessagingEndpoints::HandleFilterChanged()
{
	ReloadEndpointList();
}


void SMessagingEndpoints::HandleModelSelectedMessageChanged()
{
	TSharedPtr<FMessageTracerMessageInfo> SelectedMessage = Model->GetSelectedMessage();

	if (SelectedMessage.IsValid())
	{
		EndpointListView->SetSelection(SelectedMessage->SenderInfo);
	}
}


#undef LOCTEXT_NAMESPACE
