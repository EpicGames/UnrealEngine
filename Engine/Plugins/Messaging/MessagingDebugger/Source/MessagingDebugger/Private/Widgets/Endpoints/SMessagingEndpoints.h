// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ITableRow;
namespace ESelectInfo { enum Type : int; }
template <typename ItemType> class SListView;

class FMessagingDebuggerEndpointFilter;
class FMessagingDebuggerModel;
class IMessageTracer;
class ISlateStyle;
class STableViewBase;

struct FMessageTracerEndpointInfo;


/**
 * Implements the message endpoints panel.
 */
class SMessagingEndpoints
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessagingEndpoints) { }
	SLATE_END_ARGS()

public:

	/**  Destructor. */
	~SMessagingEndpoints();

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InModel The view model to use.
	 * @param InStyle The visual style to use for this widget.
	 * @param InTracer The message tracer.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FMessagingDebuggerModel>& InModel, const TSharedRef<ISlateStyle>& InStyle, const TSharedRef<IMessageTracer, ESPMode::ThreadSafe>& InTracer);

public:

	//~ SCompoundWidget interface

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:

	/** Reloads the list of endpoints. */
	void ReloadEndpointList();

private:

	/** Handles generating a row widget for the endpoint list view. */
	TSharedRef<ITableRow> HandleEndpointListGenerateRow(TSharedPtr<FMessageTracerEndpointInfo> EndpointInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/** Handles getting the highlight string for endpoints. */
	FText HandleEndpointListGetHighlightText() const;

	/** Handles the selection of endpoints. */
	void HandleEndpointListSelectionChanged(TSharedPtr<FMessageTracerEndpointInfo> InItem, ESelectInfo::Type SelectInfo);

	/** Handles endpoint filter changes. */
	void HandleFilterChanged();

	/** Callback for handling message selection changes. */
	void HandleModelSelectedMessageChanged();

private:

	/** Holds the filter bar. */
//	TSharedPtr<SMessagingEndpointsFilterBar> FilterBar;

	/** Holds the filtered list of historic messages. */
	TArray<TSharedPtr<FMessageTracerEndpointInfo>> EndpointList;

	/** Holds the message list view. */
	TSharedPtr<SListView<TSharedPtr<FMessageTracerEndpointInfo>>> EndpointListView;

	/** Holds the endpoint filter model. */
	TSharedPtr<FMessagingDebuggerEndpointFilter> Filter;

	/** Holds a pointer to the view model. */
	TSharedPtr<FMessagingDebuggerModel> Model;

	/** Holds the widget's visual style. */
	TSharedPtr<ISlateStyle> Style;

	/** Holds a pointer to the message bus tracer. */
	TSharedPtr<IMessageTracer, ESPMode::ThreadSafe> Tracer;
};
