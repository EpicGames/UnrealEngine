// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Endpoints/SMessagingEndpointsFilterBar.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SSearchBox.h"

#include "Models/MessagingDebuggerEndpointFilter.h"


#define LOCTEXT_NAMESPACE "SMessagingEndpointsFilterBar"


/* SMessagingEndpointsFilterBar interface
*****************************************************************************/

void SMessagingEndpointsFilterBar::Construct(const FArguments& InArgs, TSharedRef<FMessagingDebuggerEndpointFilter> InFilter)
{
	Filter = InFilter;

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Top)
			[
				// search box
				SNew(SSearchBox)
					.HintText(LOCTEXT("SearchBoxHint", "Search endpoints"))
					.OnTextChanged_Lambda([this](const FText& NewText) {
						Filter->SetFilterString(NewText.ToString());
					})
			]
	];
}


#undef LOCTEXT_NAMESPACE
