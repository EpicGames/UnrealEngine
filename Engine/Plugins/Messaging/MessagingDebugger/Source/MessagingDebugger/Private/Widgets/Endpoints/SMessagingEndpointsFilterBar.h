// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FMessagingDebuggerEndpointFilter;


/**
 * Implements the endpoints list filter bar widget.
 */
class SMessagingEndpointsFilterBar
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessagingEndpointsFilterBar) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InFilter The filter model.
	 */
	void Construct(const FArguments& InArgs, TSharedRef<FMessagingDebuggerEndpointFilter> InFilter);

private:

	/** Holds a pointer to the filter model. */
	TSharedPtr<FMessagingDebuggerEndpointFilter> Filter;
};
