// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Endpoints/SMessagingEndpointsTableRow.h"

#include "IMessageTracer.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SCheckBox.h"

#include "Models/MessagingDebuggerModel.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SMessagingEndpointsTableRow"


/* SMessagingEndpointsTableRow interface
 *****************************************************************************/

void SMessagingEndpointsTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FMessagingDebuggerModel>& InModel)
{
	check(InArgs._EndpointInfo.IsValid());
	check(InArgs._Style.IsValid());

	EndpointInfo = InArgs._EndpointInfo;
	Model = InModel;
	HighlightText = InArgs._HighlightText;
	Style = InArgs._Style;

	SMultiColumnTableRow<TSharedPtr<FMessageTracerEndpointInfo>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}


/* SMultiColumnTableRow interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SMessagingEndpointsTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == "Break")
	{
		return SNew(SBorder)
			.BorderBackgroundColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.33f))
			.BorderImage(Style->GetBrush("BreakpointBorder"));

//			return SNew(SImage)
//				.Image(Style->GetBrush("BreakDisabled"));
	}
	else if (ColumnName == "Name")
	{
		return SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.0f, 0.0f))
				[
					SNew(SImage)
						.Image(Style->GetBrush(EndpointInfo->Remote ? "RemoteEndpoint" : "LocalEndpoint"))
						.ToolTipText(EndpointInfo->Remote ? LOCTEXT("RemoteEndpointTooltip", "Remote Endpoint") : LOCTEXT("LocalEndpointTooltip", "Local Endpoint"))
				]

			+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.HighlightText(HighlightText)
						.Text(FText::FromName(EndpointInfo->Name))
				];
	}
	else if (ColumnName == "Messages")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.HighlightText(HighlightText)
					.Text_Lambda([this]() -> FText {
						return FText::AsNumber(EndpointInfo->ReceivedMessages.Num() + EndpointInfo->SentMessages.Num());
					})
					.ToolTipText_Lambda([this]() -> FText {
						return FText::Format(LOCTEXT("MessagesTooltipTextFmt", "In: {0}\nOut: {1}"),
							FText::AsNumber(EndpointInfo->ReceivedMessages.Num()),
							FText::AsNumber(EndpointInfo->SentMessages.Num()));
					})
			];
	}
	else if (ColumnName == "Visibility")
	{
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
					.Style(&Style->GetWidgetStyle<FCheckBoxStyle>("VisibilityCheckbox"))
					.IsChecked_Lambda([this]() -> ECheckBoxState {
						return (Model->IsEndpointVisible(EndpointInfo.ToSharedRef()))
							? ECheckBoxState::Checked
							: ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([this](ECheckBoxState CheckState) {
						Model->SetEndpointVisibility(EndpointInfo.ToSharedRef(), (CheckState == ECheckBoxState::Checked));
					})
					.ToolTipText(LOCTEXT("VisibilityCheckboxTooltipText", "Toggle visibility of messages from or to this endpoint"))
			];
	}

	return SNullWidget::NullWidget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE
