// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Graph/SMessagingGraph.h"



#define LOCTEXT_NAMESPACE "SMessagingGraph"


/* SMessagingGraph interface
 *****************************************************************************/

void SMessagingGraph::Construct(const FArguments& InArgs, const TSharedRef<ISlateStyle>& InStyle)
{
	ChildSlot
	[
		SNullWidget::NullWidget
//		SAssignNew(GraphEditor, SGraphEditor)
	];
}


#undef LOCTEXT_NAMESPACE
