// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ISlateStyle;


/**
 * Implements the message interaction graph panel.
 */
class SMessagingGraph
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessagingGraph) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InStyle The visual style to use for this widget.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<ISlateStyle>& InStyle);

private:

	/** Holds the graph editor widget. */
	//TSharedPtr<SGraphEditor> GraphEditor;
};
