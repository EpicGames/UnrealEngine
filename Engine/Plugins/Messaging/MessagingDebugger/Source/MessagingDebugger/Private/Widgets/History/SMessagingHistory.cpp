// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/History/SMessagingHistory.h"

#include "Framework/Views/TableViewMetadata.h"
#include "IMessageTracer.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SHyperlink.h"

#include "Models/MessagingDebuggerMessageFilter.h"
#include "Models/MessagingDebuggerModel.h"
#include "Widgets/History/SMessagingHistoryFilterBar.h"
#include "Widgets/History/SMessagingHistoryTableRow.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SMessagingHistory"


/* SMessagingHistory structors
 *****************************************************************************/

SMessagingHistory::~SMessagingHistory()
{
	if (Model.IsValid())
	{
		Model->OnMessageVisibilityChanged().RemoveAll(this);
	}

	if (Tracer.IsValid())
	{
		Tracer->OnMessageAdded().RemoveAll(this);
		Tracer->OnMessagesReset().RemoveAll(this);
	}
}


/* SMessagingHistory interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMessagingHistory::Construct(const FArguments& InArgs, const TSharedRef<FMessagingDebuggerModel>& InModel, const TSharedRef<ISlateStyle>& InStyle, const TSharedRef<IMessageTracer, ESPMode::ThreadSafe>& InTracer)
{
	Filter = MakeShareable(new FMessagingDebuggerMessageFilter());
	Model = InModel;
	ShouldScrollToLast = true;
	Style = InStyle;
	Tracer = InTracer;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
					.BorderImage(InStyle->GetBrush("GroupBorder"))
					.Padding(0.0f)
					[
						// filter bar
						SNew(SMessagingHistoryFilterBar, Filter.ToSharedRef())
					]
			]

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f, 4.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
					.BorderImage(InStyle->GetBrush("GroupBorder"))
					.Padding(0.0f)
					[
						// message list
						SAssignNew(MessageListView, SListView<TSharedPtr<FMessageTracerMessageInfo>>)
							.ListItemsSource(&MessageList)
							.SelectionMode(ESelectionMode::Single)
							.OnGenerateRow(this, &SMessagingHistory::HandleMessageListGenerateRow)
							.OnItemScrolledIntoView(this, &SMessagingHistory::HandleMessageListItemScrolledIntoView)
							.OnMouseButtonDoubleClick(this, &SMessagingHistory::HandleMessageListItemDoubleClick)
							.OnSelectionChanged(this, &SMessagingHistory::HandleMessageListSelectionChanged)
							.HeaderRow
							(
								SNew(SHeaderRow)

								+ SHeaderRow::Column("Flag")
									.DefaultLabel(FText::FromString(TEXT(" ")))
									.FixedWidth(20.0f)
									.HAlignCell(HAlign_Center)
									.HAlignHeader(HAlign_Center)
									.VAlignCell(VAlign_Center)

								+ SHeaderRow::Column("TimeSent")
									.DefaultLabel(LOCTEXT("MessageListTimeSentColumnHeader", "Time Sent"))
									.FillWidth(0.15f)
									.HAlignCell(HAlign_Right)
									.HAlignHeader(HAlign_Right)
									.VAlignCell(VAlign_Center)

								+ SHeaderRow::Column("MessageType")
									.DefaultLabel(LOCTEXT("MessageListMessageTypeColumnHeader", "Message Type"))
									.FillWidth(0.3f)
									.VAlignCell(VAlign_Center)

								+ SHeaderRow::Column("Sender")
									.DefaultLabel(LOCTEXT("MessageListSenderColumnHeader", "Sender"))
									.FillWidth(0.4f)
									.VAlignCell(VAlign_Center)

								+ SHeaderRow::Column("Recipients")
									.DefaultLabel(LOCTEXT("MessageListRecipientsColumnHeader", "Recipients"))
									.FillWidth(0.15f)
									.HAlignCell(HAlign_Center)
									.HAlignHeader(HAlign_Center)
									.VAlignCell(VAlign_Center)

								+ SHeaderRow::Column("Scope")
									.DefaultLabel(LOCTEXT("MessageListScopeColumnHeader", "Scope"))
									.FixedWidth(64.0f)
									.VAlignCell(VAlign_Center)

								+ SHeaderRow::Column("RouteLatency")
									.DefaultLabel(LOCTEXT("MessageListRouteLatencyColumnHeader", "Routing Latency"))
									.FixedWidth(112.0f)
									.HAlignCell(HAlign_Right)
									.HAlignHeader(HAlign_Right)
									.VAlignCell(VAlign_Center)

								+ SHeaderRow::Column("DispatchLatency")
									.DefaultLabel(LOCTEXT("MessageListDispatchLatencyColumnHeader", "Dispatch Latency"))
									.FixedWidth(112.0f)
									.HAlignCell(HAlign_Right)
									.HAlignHeader(HAlign_Right)
									.VAlignCell(VAlign_Center)

								+ SHeaderRow::Column("HandleTime")
									.DefaultLabel(LOCTEXT("MessageListHandleTimeColumnHeader", "Handle Time"))
									.FixedWidth(80.0f)
									.HAlignCell(HAlign_Right)
									.HAlignHeader(HAlign_Right)
									.VAlignCell(VAlign_Center)
							)
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 4.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
					.BorderImage(InStyle->GetBrush("GroupBorder"))
					.Padding(4.0f)
					[
						// status bar
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(STextBlock)
									.Text(this, &SMessagingHistory::HandleStatusBarText)
							]

						+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.HAlign(HAlign_Left)
							.Padding(8.0f, 0.0f, 0.0f, 0.0f)
							[
								SNew(SHyperlink)
									.OnNavigate(this, &SMessagingHistory::HandleShowHiddenHyperlinkNavigate)
									.Text(LOCTEXT("ShowHiddenHyperlinkText", "show all"))
									.ToolTipText(LOCTEXT("NoCulturesHyperlinkTooltip", "Reset endpoint and message type visibility filters to make all messages visible."))
									.Visibility(this, &SMessagingHistory::HandleShowHiddenHyperlinkVisibility)
							]
					]
			]
	];

	Filter->OnChanged().AddRaw(this, &SMessagingHistory::HandleFilterChanged);
	Model->OnMessageVisibilityChanged().AddRaw(this, &SMessagingHistory::HandleModelMessageVisibilityChanged);
	Tracer->OnMessageAdded().AddRaw(this, &SMessagingHistory::HandleTracerMessageAdded);
	Tracer->OnMessagesReset().AddRaw(this, &SMessagingHistory::HandleTracerMessagesReset);

	ReloadMessages();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SMessagingHistory implementation
 *****************************************************************************/

void SMessagingHistory::AddMessage(const TSharedRef<FMessageTracerMessageInfo>& MessageInfo)
{
	++TotalMessages;

	if (!Model->IsMessageVisible(MessageInfo))
	{
		return;
	}

	if (!Filter->FilterEndpoint(MessageInfo))
	{
		return;
	}

	MessageList.Add(MessageInfo);
	MessageListView->RequestListRefresh();
}


void SMessagingHistory::ReloadMessages()
{
	MessageList.Reset();
	TotalMessages = 0;
	
	TArray<TSharedPtr<FMessageTracerMessageInfo>> Messages;

	if (Tracer->GetMessages(Messages) > 0)
	{
		for (const auto& Message : Messages)
		{
			AddMessage(Message.ToSharedRef());
		}
	}

	MessageListView->RequestListRefresh();
}


/* SMessagingHistory callbacks
 *****************************************************************************/

void SMessagingHistory::HandleFilterChanged()
{
	ReloadMessages();
}


TSharedRef<ITableRow> SMessagingHistory::HandleMessageListGenerateRow(TSharedPtr<FMessageTracerMessageInfo> MessageInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SMessagingHistoryTableRow, OwnerTable)
		.HighlightText(this, &SMessagingHistory::HandleMessageListGetHighlightText)
		.MessageInfo(MessageInfo)
		.Style(Style);
}


FText SMessagingHistory::HandleMessageListGetHighlightText() const
{
	return FText::GetEmpty();
	//return FilterBar->GetFilterText();
}


void SMessagingHistory::HandleMessageListItemDoubleClick(TSharedPtr<FMessageTracerMessageInfo> Item)
{

}


void SMessagingHistory::HandleMessageListItemScrolledIntoView(TSharedPtr<FMessageTracerMessageInfo> Item, const TSharedPtr<ITableRow>& TableRow)
{
	if (MessageList.Num() > 0)
	{
		ShouldScrollToLast = MessageListView->IsItemVisible(MessageList.Last());
	}
	else
	{
		ShouldScrollToLast = true;
	}
}


void SMessagingHistory::HandleMessageListSelectionChanged(TSharedPtr<FMessageTracerMessageInfo> InItem, ESelectInfo::Type SelectInfo)
{
	Model->SelectMessage(InItem);
}


void SMessagingHistory::HandleModelMessageVisibilityChanged()
{
	ReloadMessages();
}


void SMessagingHistory::HandleShowHiddenHyperlinkNavigate()
{
	Model->ClearVisibilities();
}


EVisibility SMessagingHistory::HandleShowHiddenHyperlinkVisibility() const
{
	if (TotalMessages > MessageList.Num())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}


FText SMessagingHistory::HandleStatusBarText() const
{
	FText Result;
	int32 VisibleMessages = MessageList.Num();

	if (VisibleMessages > 0)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("NumberOfMessages"), MessageList.Num());
		Args.Add(TEXT("NumberOfSelectedMessages"), MessageListView->GetNumItemsSelected());
		Args.Add(TEXT("NumberOfHiddenMessages"), TotalMessages - VisibleMessages);

		const bool bHasSelectedMessages = MessageListView->GetNumItemsSelected() > 0;
		const bool bHasHiddenMessages = VisibleMessages < TotalMessages;

		if (bHasSelectedMessages && bHasHiddenMessages)
		{
			Result = FText::Format(LOCTEXT("StatusBar Number Messages, Selected Messages and Hidden Messages", "{NumberOfMessages} messages, {NumberOfSelectedMessages} selected, {NumberOfHiddenMessages} hidden"), Args);
		}
		else if (bHasSelectedMessages && !bHasHiddenMessages)
		{
			Result = FText::Format(LOCTEXT("StatusBar Number Messages and Selected Messages", "{NumberOfMessages} messages, {NumberOfSelectedMessages} selected"), Args);
		}
		else if (!bHasSelectedMessages && bHasHiddenMessages)
		{
			Result = FText::Format(LOCTEXT("StatusBar Number Messages and Hidden Messages", "{NumberOfMessages} messages, {NumberOfHiddenMessages} hidden"), Args);
		}
		else //if(!bHasSelectedMessages && !bHasHiddenMessages)
		{
			Result = FText::Format(LOCTEXT("StatusBar Number Messages", "{NumberOfMessages} messages"), Args);
		}
	}
	else
	{
		Result = LOCTEXT("StatusBarBeginTracing", "Press the 'Start' button to trace messages");
	}

	return Result;
}


void SMessagingHistory::HandleTracerMessageAdded(TSharedRef<FMessageTracerMessageInfo> MessageInfo)
{
	AddMessage(MessageInfo);

	if (ShouldScrollToLast && !Tracer->IsBreaking() && (MessageList.Num() > 0))
	{
		MessageListView->RequestScrollIntoView(MessageList.Last(0));
	}
}


void SMessagingHistory::HandleTracerMessagesReset()
{
	ReloadMessages();
}


#undef LOCTEXT_NAMESPACE
