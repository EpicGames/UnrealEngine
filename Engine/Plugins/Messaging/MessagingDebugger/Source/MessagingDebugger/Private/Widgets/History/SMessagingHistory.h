// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ITableRow;
namespace ESelectInfo { enum Type : int; }
template <typename ItemType> class SListView;

class FMessagingDebuggerMessageFilter;
class FMessagingDebuggerModel;
class IMessageTracer;
class ISlateStyle;
class STableViewBase;

struct FMessageTracerMessageInfo;


/**
 * Implements the message history panel.
 */
class SMessagingHistory
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessagingHistory) { }
	SLATE_END_ARGS()

public:

	/** Destructor. */
	~SMessagingHistory();

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InModel The view model to use.
	 * @param InStyle The visual style to use for this widget.
	 * @param InTracer The message tracer.
	 */
	void Construct(
		const FArguments& InArgs,
		const TSharedRef<FMessagingDebuggerModel>& InModel,
		const TSharedRef<ISlateStyle>& InStyle,
		const TSharedRef<IMessageTracer, ESPMode::ThreadSafe>& InTracer);

protected:

	/**
	 * Adds the given message to the history.
	 *
	 * @param MessageInfo The information about the message to add.
	 */
	void AddMessage(const TSharedRef<FMessageTracerMessageInfo>& MessageInfo);

	/** Reloads the message history. */
	void ReloadMessages();

private:

	/** Callback for message filter changes. */
	void HandleFilterChanged();

	/** Callback for generating a row widget for the message list view. */
	TSharedRef<ITableRow> HandleMessageListGenerateRow(TSharedPtr<FMessageTracerMessageInfo> MessageInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback for getting the message list's highlight string. */
	FText HandleMessageListGetHighlightText() const;

	/** Callback for double-clicking a message item. */
	void HandleMessageListItemDoubleClick(TSharedPtr<FMessageTracerMessageInfo> Item);

	/** Callback for scrolling a message item into view. */
	void HandleMessageListItemScrolledIntoView(TSharedPtr<FMessageTracerMessageInfo> Item, const TSharedPtr<ITableRow>& TableRow);

	/** Callback for selecting a message in the message list view. */
	void HandleMessageListSelectionChanged(TSharedPtr<FMessageTracerMessageInfo> InItem, ESelectInfo::Type SelectInfo);

	/** Callback for changes in message visibility. */
	void HandleModelMessageVisibilityChanged();

	/** Callback for clicking the 'Show hidden' hyperlink. */
	void HandleShowHiddenHyperlinkNavigate();

	/** Callback for getting the visibility of the 'Show hidden' hyperlink. */
	EVisibility HandleShowHiddenHyperlinkVisibility() const;

	/** Callback for getting the status bar text. */
	FText HandleStatusBarText() const;

	/** Callback for when a message has been added to the message tracer. */
	void HandleTracerMessageAdded(TSharedRef<FMessageTracerMessageInfo> MessageInfo);

	/** Callback for when the tracer's message history has been reset. */
	void HandleTracerMessagesReset();

private:

	/** Holds the message filter model. */
	TSharedPtr<FMessagingDebuggerMessageFilter> Filter;

	/** Holds the filtered list of historic messages. */
	TArray<TSharedPtr<FMessageTracerMessageInfo>> MessageList;

	/** Holds the message list view. */
	TSharedPtr<SListView<TSharedPtr<FMessageTracerMessageInfo>>> MessageListView;

	/** Holds a pointer to the view model. */
	TSharedPtr<FMessagingDebuggerModel> Model;

	/** Holds a flag indicating whether the message list should auto-scroll to the last item. */
	bool ShouldScrollToLast;

	/** Holds the widget's visual style. */
	TSharedPtr<ISlateStyle> Style;

	/** Holds the total number of messages (including invisible ones). */
	int32 TotalMessages;

	/** Holds a pointer to the message bus tracer. */
	TSharedPtr<IMessageTracer, ESPMode::ThreadSafe> Tracer;
};
