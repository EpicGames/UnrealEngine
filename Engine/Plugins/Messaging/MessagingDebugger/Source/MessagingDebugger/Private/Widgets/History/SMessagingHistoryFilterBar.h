// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"


class FMessagingDebuggerMessageFilter;


#define LOCTEXT_NAMESPACE "SMessagingHistoryFilterBar"


/**
 * Implements the messaging history command bar widget.
 */
class SMessagingHistoryFilterBar
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessagingHistoryFilterBar) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InFilter The filter model.
	 */
	void Construct(const FArguments& InArgs, TSharedRef<FMessagingDebuggerMessageFilter> InFilter)
	{
		ChildSlot
		[
			SNullWidget::NullWidget
		];
	}

private:

	/** Holds the filter model. */
	TSharedPtr<FMessagingDebuggerMessageFilter> Filter;
};


#undef LOCTEXT_NAMESPACE
