// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/History/SMessagingHistoryTableRow.h"

#include "IMessageTracer.h"
#include "SlateOptMacros.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SMessagingHistoryTableRow"


/* SMessagingHistoryTableRow interface
 *****************************************************************************/

void SMessagingHistoryTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	check(InArgs._MessageInfo.IsValid());
	check(InArgs._Style.IsValid());

	MaxDispatchLatency = -1.0;
	MaxHandlingTime = -1.0;
	HighlightText = InArgs._HighlightText;
	MessageInfo = InArgs._MessageInfo;
	Style = InArgs._Style;

	SMultiColumnTableRow<TSharedPtr<FMessageTracerMessageInfo>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}


/* SWidget interface
 *****************************************************************************/

void SMessagingHistoryTableRow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	MaxDispatchLatency = -1.0;
	MaxHandlingTime = -1.0;

	for (const auto& DispatchStatePair : MessageInfo->DispatchStates)
	{
		const TSharedPtr<FMessageTracerDispatchState>& DispatchState = DispatchStatePair.Value;

		if (MessageInfo->TimeRouted > 0.0)
		{
			MaxDispatchLatency = FMath::Max(MaxDispatchLatency, DispatchState->DispatchLatency);
		}

		if (DispatchState->TimeHandled > 0.0)
		{
			MaxHandlingTime = FMath::Max(MaxHandlingTime, DispatchState->TimeHandled - DispatchState->TimeDispatched);
		}
	}
}


/* SMultiColumnTableRow interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SMessagingHistoryTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == "DispatchLatency")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
					.ColorAndOpacity(this, &SMessagingHistoryTableRow::HandleTextColorAndOpacity)
					.Text_Lambda([this]() -> FText {
						return TimespanToReadableText(MaxDispatchLatency);
					})
			];
	}
	else if (ColumnName == "Flag")
	{
		return SNew(SBox)
			.ToolTipText(LOCTEXT("DeadMessageTooltip", "No valid recipients (dead letter)"))
			[
				SNew(SImage)
					.Image(this, &SMessagingHistoryTableRow::HandleFlagImage)
			];
	}
	else if (ColumnName == "HandleTime")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
					.ColorAndOpacity(this, &SMessagingHistoryTableRow::HandleTextColorAndOpacity)
					.Text_Lambda([this]() -> FText {
						return TimespanToReadableText(MaxHandlingTime);
					})
			];
	}
	else if (ColumnName == "MessageType")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
					.ColorAndOpacity(this, &SMessagingHistoryTableRow::HandleTextColorAndOpacity)
					.HighlightText(HighlightText)
					.Text(FText::FromName(MessageInfo->Context->GetMessageTypePathName().GetAssetName()))
			];
	}
	else if (ColumnName == "Recipients")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			.ToolTipText(LOCTEXT("LocalRemoteRecipients", "Local/Remote Recipients"))
			[
				SNew(STextBlock)
					.ColorAndOpacity(this, &SMessagingHistoryTableRow::HandleTextColorAndOpacity)
					.Text(this, &SMessagingHistoryTableRow::HandleRecipientsText)
			];
	}
	else if (ColumnName == "RouteLatency")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
					.ColorAndOpacity(this, &SMessagingHistoryTableRow::HandleRouteLatencyColorAndOpacity)
					.Text(this, &SMessagingHistoryTableRow::HandleRouteLatencyText)
			];
	}
	else if (ColumnName == "Scope")
	{
		FText Text;
		FText ToolTipText;

		if (MessageInfo->Context.IsValid())
		{
			int32 NumRecipients = MessageInfo->Context->GetRecipients().Num();

			if (MessageInfo->Context->IsForwarded())
			{
				// forwarded message
				FText ScopeText = ScopeToText(MessageInfo->Context->GetScope());
				Text = FText::Format(LOCTEXT("ForwardedMessageTextFormat", "F - {0}"), ScopeText);

				if (NumRecipients > 0)
				{
					ToolTipText = FText::Format(LOCTEXT("ForwardedSentMessageToolTipTextFormat", "This message was forwarded directly to {0} recipients"), FText::AsNumber(NumRecipients));
				}
				else
				{
					ToolTipText = FText::Format(LOCTEXT("ForwardedPublishedMessageToolTipTextFormat", "This message was forwarded to all subscribed recipients in scope '{0}'"), ScopeText);
				}				
			}
			else if (NumRecipients == 0)
			{
				// published message
				FText ScopeText = ScopeToText(MessageInfo->Context->GetScope());

				Text = FText::Format(LOCTEXT("PublishedMessageTextFormat", "P - {0}"), ScopeText);
				ToolTipText = FText::Format(LOCTEXT("PublishedMessageToolTipTextFormat", "This message was published to all subscribed recipients in scope '{0}'"), ScopeText);
			}
			else
			{
				// sent message
				Text = LOCTEXT("SentMessageTextFormat", "S");
				ToolTipText = FText::Format(LOCTEXT("SentMessageToolTipTextFormat", "This message was sent directly to {0} recipients"), FText::AsNumber(NumRecipients));
			}
		}

		return SNew(SBox)
			.ToolTipText(ToolTipText)
			[
				SNew(STextBlock)
					.ColorAndOpacity(this, &SMessagingHistoryTableRow::HandleTextColorAndOpacity)
					.Text(Text)
			];
	}
	else if (ColumnName == "Sender")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
					.ColorAndOpacity(this, &SMessagingHistoryTableRow::HandleTextColorAndOpacity)
					.HighlightText(HighlightText)
					.Text_Lambda([this]() -> FText {
						return FText::FromString(MessageInfo->SenderInfo->Name.ToString());
					})
			];
	}
	else if (ColumnName == "TimeSent")
	{
		FNumberFormattingOptions NumberFormattingOptions;
		NumberFormattingOptions.MaximumFractionalDigits = 3;
		NumberFormattingOptions.MinimumFractionalDigits = 3;

		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
					.ColorAndOpacity(this, &SMessagingHistoryTableRow::HandleTextColorAndOpacity)
					.HighlightText(HighlightText)
					.Text(FText::AsNumber(MessageInfo->TimeSent - GStartTime, &NumberFormattingOptions))
			];
	}

	return SNullWidget::NullWidget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SMessagingHistoryTableRow implementation
 *****************************************************************************/

FText SMessagingHistoryTableRow::ScopeToText(EMessageScope Scope) const
{
	switch (Scope)
	{
	case EMessageScope::Thread:
		return LOCTEXT("ScopeThread", "Thread");

	case EMessageScope::Process:
		return LOCTEXT("ScopeProcess", "Process");

	case EMessageScope::Network:
		return LOCTEXT("ScopeNetwork", "Network");

	case EMessageScope::All:
		return LOCTEXT("ScopeAll", "All");

	default:
		return LOCTEXT("ScopeUnknown", "Unknown");
	}
}


FText SMessagingHistoryTableRow::TimespanToReadableText(double Seconds) const
{
	if (Seconds < 0.0)
	{
		return LOCTEXT("Zero Length Timespan", "-");
	}

	FNumberFormattingOptions Options;
	Options.MinimumFractionalDigits = 1;
	Options.MaximumFractionalDigits = 1;

	if (Seconds < 0.0001)
	{
		return FText::Format(LOCTEXT("Seconds < 0.0001 Length Timespan", "{0} us"), FText::AsNumber(Seconds * 1000000, &Options));
	}

	if (Seconds < 0.1)
	{
		return FText::Format(LOCTEXT("Seconds < 0.1 Length Timespan", "{0} ms"), FText::AsNumber(Seconds * 1000, &Options));
	}

	if (Seconds < 60.0)
	{
		return FText::Format(LOCTEXT("Seconds < 60.0 Length Timespan", "{0} s"), FText::AsNumber(Seconds, &Options));
	}
		
	return LOCTEXT("> 1 minute Length Timespan", "> 1 min");
}


/* SMessagingHistoryTableRow callbacks
 *****************************************************************************/

FSlateColor SMessagingHistoryTableRow::HandleDispatchLatencyColorAndOpacity() const
{
	if (MaxDispatchLatency >= 0.01)
	{
		return FLinearColor::Red;
	}

	if (MaxDispatchLatency >= 0.001)
	{
		return FLinearColor(1.0f, 1.0f, 0.0f);
	}

	if (MaxDispatchLatency >= 0.0001)
	{
		return FLinearColor::Yellow;
	}

	return FSlateColor::UseForeground();
}


const FSlateBrush* SMessagingHistoryTableRow::HandleFlagImage() const
{
	if ((MessageInfo->TimeRouted > 0.0) && (MessageInfo->DispatchStates.Num() == 0))
	{
		return Style->GetBrush("DeadMessage");
	}

	return nullptr;
}


FText SMessagingHistoryTableRow::HandleRecipientsText() const
{
	int32 LocalRecipients = 0;
	int32 RemoteRecipients = 0;

	for (auto It = MessageInfo->DispatchStates.CreateConstIterator(); It; ++It)
	{
		TSharedPtr<FMessageTracerEndpointInfo> EndpointInfo = It.Value()->EndpointInfo;

		if (EndpointInfo.IsValid())
		{
			if (EndpointInfo->Remote)
			{
				++RemoteRecipients;
			}
			else
			{
				++LocalRecipients;
			}
		}
	}

	return FText::Format(LOCTEXT("RecipientsTextFormat", "{0} / {1}"), FText::AsNumber(LocalRecipients), FText::AsNumber(RemoteRecipients));
}


FSlateColor SMessagingHistoryTableRow::HandleRouteLatencyColorAndOpacity() const
{
	double RouteLatency = MessageInfo->TimeRouted - MessageInfo->TimeSent;

	if (RouteLatency >= 0.01)
	{
		return FLinearColor::Red;
	}

	if (RouteLatency >= 0.001)
	{
		return FLinearColor(1.0f, 1.0f, 0.0f);
	}

	if (RouteLatency >= 0.0001)
	{
		return FLinearColor::Yellow;
	}

	if (MessageInfo->TimeRouted == 0.0)
	{
		return FSlateColor::UseSubduedForeground();
	}

	return FSlateColor::UseForeground();
}


FText SMessagingHistoryTableRow::HandleRouteLatencyText() const
{
	if (MessageInfo->TimeRouted > 0.0)
	{
		return TimespanToReadableText(MessageInfo->TimeRouted - MessageInfo->TimeSent);
	}

	return LOCTEXT("RouteLatencyPending", "Pending");
}


FSlateColor SMessagingHistoryTableRow::HandleTextColorAndOpacity() const
{
	if (MessageInfo->TimeRouted == 0.0)
	{
		return FSlateColor::UseSubduedForeground();
	}

	return FSlateColor::UseForeground();
}


#undef LOCTEXT_NAMESPACE
