// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class IMessageTracer;
class ISlateStyle;
class STableViewBase;

struct FMessageTracerMessageInfo;
struct FSlateBrush;

enum class EMessageScope : uint8;


/**
 * Implements a row widget for the message history list.
 */
class SMessagingHistoryTableRow
	: public SMultiColumnTableRow<TSharedPtr<FMessageTracerMessageInfo>>
{
public:

	SLATE_BEGIN_ARGS(SMessagingHistoryTableRow) { }
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(TSharedPtr<FMessageTracerMessageInfo>, MessageInfo)
		SLATE_ARGUMENT(TSharedPtr<ISlateStyle>, Style)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InOwnerTableView The table view that owns this row.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

public:

	//~ SWidget overrides

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

public:

	//~ SMultiColumnTableRow interface

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

protected:

	/** 
	 * Converts the given message scope to a human readable string.
	 *
	 * @return The string representation.
	 */
	FText ScopeToText(EMessageScope Scope) const;

	/**
	 * Converts the given time span in seconds to a human readable string.
	 *
	 * @param Seconds The time span to convert.
	 * @return The string representation.
	 *
	 * @todo gmp: refactor this into FText::AsTimespan or something like that
	 */
	FText TimespanToReadableText(double Seconds) const;

private:

	/** Callback for getting the text color of the DispatchLatency column. */
	FSlateColor HandleDispatchLatencyColorAndOpacity() const;

	const FSlateBrush* HandleFlagImage() const;

	/** Callback for getting the text in the Recipients column. */
	FText HandleRecipientsText() const;

	/** Callback for getting the text color of the RouteLatency column. */
	FSlateColor HandleRouteLatencyColorAndOpacity() const;

	/** Callback for getting the text in the RouteLatency column. */
	FText HandleRouteLatencyText() const;

	/** Callback for getting the text color of various columns. */
	FSlateColor HandleTextColorAndOpacity() const;

private:

	/** Holds the highlight string for the message. */
	TAttribute<FText> HighlightText;

	/** Holds message's debug information. */
	TSharedPtr<FMessageTracerMessageInfo> MessageInfo;

	/** Holds the maximum dispatch latency. */
	double MaxDispatchLatency;

	/** Holds the maximum time that was needed to handle the message. */
	double MaxHandlingTime;

	/** Holds the widget's visual style. */
	TSharedPtr<ISlateStyle> Style;
};
