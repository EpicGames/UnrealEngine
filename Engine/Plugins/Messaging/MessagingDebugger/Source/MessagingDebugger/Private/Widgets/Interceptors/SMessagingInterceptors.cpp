// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Interceptors/SMessagingInterceptors.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SlateOptMacros.h"
#include "IMessageTracer.h"
#include "Widgets/Views/SListView.h"
#include "MessagingDebuggerModel.h"
#include "Widgets/Interceptors/SMessagingInterceptorsTableRow.h"


#define LOCTEXT_NAMESPACE "SMessagingInterceptors"


/* SMessagingInterceptors interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMessagingInterceptors::Construct(const FArguments& InArgs, const TSharedRef<FMessagingDebuggerModel>& InModel, const TSharedRef<ISlateStyle>& InStyle, const TSharedRef<IMessageTracer, ESPMode::ThreadSafe>& InTracer)
{
	Model = InModel;
	Style = InStyle;
	Tracer = InTracer;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f, 4.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
					.BorderImage(InStyle->GetBrush("GroupBorder"))
					.Padding(0.0f)
					[
						// interceptor list
						SAssignNew(InterceptorListView, SListView<TSharedPtr<FMessageTracerInterceptorInfo>>)
							.ListItemsSource(&InterceptorList)
							.SelectionMode(ESelectionMode::None)
							.OnGenerateRow(this, &SMessagingInterceptors::HandleInterceptorListGenerateRow)
							.HeaderRow
							(
								SNew(SHeaderRow)

								+ SHeaderRow::Column("Name")
									.DefaultLabel(FText::FromString(TEXT("Interceptors")))
									.FillWidth(1.0f)

								+ SHeaderRow::Column("TimeRegistered")
									.DefaultLabel(LOCTEXT("InterceptorListTimeRegisteredColumnHeader", "Time Registered"))
									.FixedWidth(112.0f)
									.HAlignCell(HAlign_Right)
									.HAlignHeader(HAlign_Right)

								+ SHeaderRow::Column("TimeUnregistered")
									.DefaultLabel(LOCTEXT("InterceptorListTimeUnregisteredColumnHeader", "Time Unregistered"))
									.FixedWidth(112.0f)
									.HAlignCell(HAlign_Right)
									.HAlignHeader(HAlign_Right)
							)
					]
			]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SMessagingInterceptors implementation
 *****************************************************************************/

void SMessagingInterceptors::ReloadInterceptorList()
{
	InterceptorList.Empty();
}


/* SMessagingInterceptors callbacks
 *****************************************************************************/

TSharedRef<ITableRow> SMessagingInterceptors::HandleInterceptorListGenerateRow(TSharedPtr<FMessageTracerInterceptorInfo> InterceptorInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SMessagingInterceptorsTableRow, OwnerTable, Model.ToSharedRef())
		.InterceptorInfo(InterceptorInfo)
		.Style(Style);
//		.ToolTipText(EndpointInfo->Address.ToString(EGuidFormats::DigitsWithHyphensInBraces));
}


#undef LOCTEXT_NAMESPACE
