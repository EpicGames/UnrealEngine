// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FMessagingDebuggerModel;
class IMessageTracer;
class ISlateStyle;
class ITableRow;
class STableViewBase;
struct FMessageTracerInterceptorInfo;
template <typename ItemType> class SListView;

/**
 * Implements the message interceptors panel.
 */
class SMessagingInterceptors
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessagingInterceptors) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InModel The view model to use.
	 * @param InStyle The visual style to use for this widget.
	 * @param InTracer The message tracer.
	 */
	void Construct(
		const FArguments& InArgs,
		const TSharedRef<FMessagingDebuggerModel>& InModel,
		const TSharedRef<ISlateStyle>& InStyle,
		const TSharedRef<IMessageTracer, ESPMode::ThreadSafe>& InTracer);

protected:

	/** Reloads the list of interceptors. */
	void ReloadInterceptorList();

private:

	/** Handles generating a row widget for the interceptor list view. */
	TSharedRef<ITableRow> HandleInterceptorListGenerateRow(TSharedPtr<FMessageTracerInterceptorInfo> InterceptorInfo, const TSharedRef<STableViewBase>& OwnerTable);

private:

	/** Holds the filtered list of interceptors. */
	TArray<TSharedPtr<FMessageTracerInterceptorInfo>> InterceptorList;

	/** Holds the interceptor list view. */
	TSharedPtr<SListView<TSharedPtr<FMessageTracerInterceptorInfo>>> InterceptorListView;

	/** Holds a pointer to the view model. */
	TSharedPtr<FMessagingDebuggerModel> Model;

	/** Holds the widget's visual style. */
	TSharedPtr<ISlateStyle> Style;

	/** Holds a pointer to the message bus tracer. */
	TSharedPtr<IMessageTracer, ESPMode::ThreadSafe> Tracer;
};
