// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Interceptors/SMessagingInterceptorsTableRow.h"

#include "IMessageTracer.h"

#include "Models/MessagingDebuggerModel.h"
#include "SlateOptMacros.h"
#include "Widgets/Views/SListView.h"


namespace MessagingInterceptorTableRow
{
	static const FNumberFormattingOptions TimeRegisteredFormattingOptions = FNumberFormattingOptions()
		.SetMinimumFractionalDigits(5)
		.SetMaximumFractionalDigits(5);
}


#define LOCTEXT_NAMESPACE "SMessagingInterceptorTableRow"


/* SMessagingInterceptorsTableRow interface
 *****************************************************************************/

void SMessagingInterceptorsTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FMessagingDebuggerModel>& InModel)
{
	check(InArgs._Style.IsValid());
	check(InArgs._InterceptorInfo.IsValid());

	InterceptorInfo = InArgs._InterceptorInfo;
	Model = InModel;
	Style = InArgs._Style;

	SMultiColumnTableRow<TSharedPtr<FMessageTracerInterceptorInfo>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}


/* SMultiColumnTableRow interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SMessagingInterceptorsTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == "Name")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(FText::FromString(InterceptorInfo->Name.ToString()))
			];
	}
	else if (ColumnName == "TimeRegistered")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(FText::AsNumber(InterceptorInfo->TimeRegistered - GStartTime, &MessagingInterceptorTableRow::TimeRegisteredFormattingOptions))
			];
	}
	else if (ColumnName == "TimeUnregistered")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(this, &SMessagingInterceptorsTableRow::HandleTimeUnregisteredText)
			];
	}

	return SNullWidget::NullWidget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


FText SMessagingInterceptorsTableRow::HandleTimeUnregisteredText() const
{
	if (InterceptorInfo->TimeUnregistered > 0.0)
	{
		return FText::AsNumber(InterceptorInfo->TimeUnregistered - GStartTime, &MessagingInterceptorTableRow::TimeRegisteredFormattingOptions);
	}

	return LOCTEXT("Never", "Never");
}


#undef LOCTEXT_NAMESPACE
