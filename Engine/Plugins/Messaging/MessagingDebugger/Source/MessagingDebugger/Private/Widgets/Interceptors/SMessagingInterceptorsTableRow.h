// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FMessagingDebuggerModel;
class ISlateStyle;
class STableViewBase;

struct FMessageTracerInterceptorInfo;


/**
 * Implements a row widget for the interceptors list.
 */
class SMessagingInterceptorsTableRow
	: public SMultiColumnTableRow<TSharedPtr<FMessageTracerInterceptorInfo>>
{
public:

	SLATE_BEGIN_ARGS(SMessagingInterceptorsTableRow) { }
		SLATE_ARGUMENT(TSharedPtr<FMessageTracerInterceptorInfo>, InterceptorInfo)
		SLATE_ARGUMENT(TSharedPtr<ISlateStyle>, Style)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InOwnerTableView The table view that owns this row.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FMessagingDebuggerModel>& InModel);

public:

	//~ SMultiColumnTableRow interface

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	/** Callback for getting the timestamp at which the address was unregistered. */
	FText HandleTimeUnregisteredText() const;

private:

	/** Holds the interceptor information. */
	TSharedPtr<FMessageTracerInterceptorInfo> InterceptorInfo;

	/** Holds a pointer to the view model. */
	TSharedPtr<FMessagingDebuggerModel> Model;

	/** Holds the widget's visual style. */
	TSharedPtr<ISlateStyle> Style;
};
