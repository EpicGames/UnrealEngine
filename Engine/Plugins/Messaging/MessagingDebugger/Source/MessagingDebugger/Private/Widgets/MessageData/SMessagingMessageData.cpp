// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/MessageData/SMessagingMessageData.h"

#include "DetailsViewArgs.h"
#include "Modules/ModuleManager.h"
#include "IMessageTracer.h"
#include "Styling/ISlateStyle.h"
#include "UObject/StructOnScope.h"

#if WITH_EDITOR
	#include "IDetailsView.h"
	#include "IStructureDetailsView.h"
	#include "PropertyEditorModule.h"
#endif

#include "Models/MessagingDebuggerModel.h"


#define LOCTEXT_NAMESPACE "SMessagingMessageData"


/* SMessagingMessageData structors
 *****************************************************************************/

SMessagingMessageData::~SMessagingMessageData()
{
	if (Model.IsValid())
	{
		Model->OnSelectedMessageChanged().RemoveAll(this);
	}
}


/* SMessagingMessageData interface
 *****************************************************************************/

void SMessagingMessageData::Construct(const FArguments& InArgs, const TSharedRef<FMessagingDebuggerModel>& InModel, const TSharedRef<ISlateStyle>& InStyle)
{
	Model = InModel;
	Style = InStyle;

#if WITH_EDITOR

	// initialize details view
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.NotifyHook = this;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.bForceHiddenPropertyVisibility = true;
	}

	FStructureDetailsViewArgs StructureViewArgs;
	{
		StructureViewArgs.bShowObjects = false;
		StructureViewArgs.bShowAssets = true;
		StructureViewArgs.bShowClasses = true;
		StructureViewArgs.bShowInterfaces = false;
	}

	StructureDetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor")
		.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr, LOCTEXT("MessageData", "Message Data"));
	{
		IDetailsView* DetailsView = StructureDetailsView->GetDetailsView();
		DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &SMessagingMessageData::HandleDetailsViewIsPropertyEditable));
		DetailsView->SetVisibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SMessagingMessageData::HandleDetailsViewVisibility)));
	}

	ChildSlot
	[
		StructureDetailsView->GetWidget().ToSharedRef()
	];

#else

	ChildSlot
	[
		SAssignNew(TextBox, SMultiLineEditableTextBox)
			.IsReadOnly(true)
	];

#endif //WITH_EDITOR

	Model->OnSelectedMessageChanged().AddRaw(this, &SMessagingMessageData::HandleModelSelectedMessageChanged);
}


/* FNotifyHook interface
 *****************************************************************************/

void SMessagingMessageData::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged)
{
}


/* SMessagingMessageData callbacks
 *****************************************************************************/

bool SMessagingMessageData::HandleDetailsViewIsPropertyEditable() const
{
	TSharedPtr<FMessageTracerMessageInfo> SelectedMessage = Model->GetSelectedMessage();

	if (!SelectedMessage.IsValid() || !SelectedMessage->Context.IsValid())
	{
		return false;
	}

	return (SelectedMessage->TimeRouted == 0.0);
}


EVisibility SMessagingMessageData::HandleDetailsViewVisibility() const
{
	if (Model->GetSelectedMessage().IsValid())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}


void SMessagingMessageData::HandleModelSelectedMessageChanged()
{
	TSharedPtr<FMessageTracerMessageInfo> SelectedMessage = Model->GetSelectedMessage();

	if (SelectedMessage.IsValid() && SelectedMessage->Context.IsValid())
	{
		UScriptStruct* MessageTypeInfo = SelectedMessage->Context->GetMessageTypeInfo().Get();

		if (MessageTypeInfo != nullptr)
		{
#if WITH_EDITOR
			StructureDetailsView->SetStructureData(MakeShareable(new FStructOnScope(MessageTypeInfo, (uint8*)SelectedMessage->Context->GetMessage())));
#else
			FBufferArchive BufferArchive;
			FJsonStructSerializerBackend Backend(BufferArchive, EStructSerializerBackendFlags::Default);

			FStructSerializer::Serialize(SelectedMessage->Context->GetMessage(), *MessageTypeInfo, Backend);

			// add string terminator
			BufferArchive.Add(0);
			BufferArchive.Add(0);

			TextBox->SetText(FText::FromString(FString((TCHAR*)BufferArchive.GetData()).Replace(TEXT("\t"), TEXT("    "))));
#endif //WITH_EDITOR
		}
		else
		{
#if WITH_EDITOR
			StructureDetailsView->SetStructureData(nullptr);
#else
			TextBox->SetText(FText::Format(LOCTEXT("UnknownMessageTypeFormat", "Unknown message type '{0}'"), FText::FromString(SelectedMessage->Context->GetMessageType().ToString())));
#endif //WITH_EDITOR
		}
	}
	else
	{
#if WITH_EDITOR
		StructureDetailsView->SetStructureData(nullptr);
#else
		TextBox->SetText(FText::GetEmpty());
#endif //WITH_EDITOR
	}
}


#undef LOCTEXT_NAMESPACE
