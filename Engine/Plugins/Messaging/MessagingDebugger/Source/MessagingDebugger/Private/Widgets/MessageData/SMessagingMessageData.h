// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Widgets/SCompoundWidget.h"

class FMessagingDebuggerModel;
class ISlateStyle;
class IStructureDetailsView;
class SMultiLineEditableTextBox;

struct FPropertyChangedEvent;


/**
 * Implements the message data panel.
 */
class SMessagingMessageData
	: public SCompoundWidget
	, public FNotifyHook
{
public:

	SLATE_BEGIN_ARGS(SMessagingMessageData) { }
	SLATE_END_ARGS()

public:

	/** Destructor. */
	~SMessagingMessageData();

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InModel The view model to use.
	 * @param InStyle The visual style to use for this widget.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FMessagingDebuggerModel>& InModel, const TSharedRef<ISlateStyle>& InStyle);

public:

	//~ FNotifyHook interface

	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged) override;

private:

	/** Handles checking whether the details view is editable. */
	bool HandleDetailsViewIsPropertyEditable() const;

	/** Handles determining the visibility of the details view. */
	EVisibility HandleDetailsViewVisibility() const;

	/** Callback for handling message selection changes. */
	void HandleModelSelectedMessageChanged();

private:

#if WITH_EDITOR
	/** Holds the structure details view. */
	TSharedPtr<IStructureDetailsView> StructureDetailsView;
#else
	/** Holds the details text box. */
	TSharedPtr<SMultiLineEditableTextBox> TextBox;
#endif //WITH_EDITOR

	/** Holds a pointer to the view model. */
	TSharedPtr<FMessagingDebuggerModel> Model;

	/** Holds the widget's visual style. */
	TSharedPtr<ISlateStyle> Style;
};
