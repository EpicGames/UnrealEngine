// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageTracer.h"
#include "SlateOptMacros.h"
#include "Widgets/Views/SListView.h"

#include "Models/MessagingDebuggerModel.h"


#define LOCTEXT_NAMESPACE "SMessagingDispatchStateTableRow"


/**
 * Implements a row widget for the dispatch state list.
 */
class SMessagingDispatchStateTableRow
	: public SMultiColumnTableRow<TSharedPtr<FMessageTracerDispatchState>>
{
public:

	SLATE_BEGIN_ARGS(SMessagingDispatchStateTableRow) { }
		SLATE_ARGUMENT(TSharedPtr<FMessageTracerDispatchState>, DispatchState)
		SLATE_ARGUMENT(TSharedPtr<ISlateStyle>, Style)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InOwnerTableView The table view that owns this row.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FMessagingDebuggerModel>& InModel)
	{
		check(InArgs._Style.IsValid());
		check(InArgs._DispatchState.IsValid());

		DispatchState = InArgs._DispatchState;
		Model = InModel;
		Style = InArgs._Style;

		SMultiColumnTableRow<TSharedPtr<FMessageTracerDispatchState>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	//~ SMultiColumnTableRow interface

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == "DispatchLatency")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(TimespanToReadableText(DispatchState->DispatchLatency))
				];
		}
		else if (ColumnName == "DispatchType")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(this, &SMessagingDispatchStateTableRow::HandleDispatchTypeText)
						.ToolTipText(this, &SMessagingDispatchStateTableRow::HandleDispatchTypeTooltip)
				];
		}
		else if (ColumnName == "HandleTime")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(this, &SMessagingDispatchStateTableRow::HandleHandlingTimeColorAndOpacity)
						.Text(this, &SMessagingDispatchStateTableRow::HandleHandlingTimeText)
				];
		}
		else if (ColumnName == "Recipient")
		{
			TSharedPtr<FMessageTracerEndpointInfo> EndpointInfo = DispatchState->EndpointInfo;

			return SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.0f, 0.0f))
				[
					SNew(SImage)
						.Image(Style->GetBrush(EndpointInfo->Remote ? "RemoteEndpoint" : "LocalEndpoint"))
						.ToolTipText(EndpointInfo->Remote ? LOCTEXT("RemoteEndpointTooltip", "Remote Endpoint") : LOCTEXT("LocalEndpointTooltip", "Local Endpoint"))
				]

			+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(FText::FromName(EndpointInfo->Name))
				];
		}
		else if (ColumnName == "RecipientThread")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(NamedThreadToReadableText(DispatchState->RecipientThread))
				];
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

protected:

	/**
	 * Converts a time span a color value.
	 *
	 * @param Latency The time span to convert.
	 * @return The corresponding color value.
	 */
	FSlateColor TimespanToColor(double Timespan) const
	{
		if (Timespan >= 0.01)
		{
			return FLinearColor::Red;
		}

		if (Timespan >= 0.001)
		{
			return FLinearColor(1.0f, 1.0f, 0.0f);
		}

		if (Timespan >= 0.0001)
		{
			return FLinearColor::Yellow;
		}

		return FSlateColor::UseForeground();
	}

	/**
	 * Converts a named thread to a human readable string.
	 *
	 * @param NamedThread The named thread.
	 * @return The text representation.
	 */
	FText NamedThreadToReadableText(ENamedThreads::Type NamedThread)
	{
		switch (NamedThread)
		{
		case ENamedThreads::AnyThread:
			return LOCTEXT("AnyThread", "AnyThread");
			break;

		case ENamedThreads::RHIThread:
			return LOCTEXT("RHIThread", "RHIThread");
			break;

		case ENamedThreads::GameThread:
			return LOCTEXT("GameThread", "GameThread");
			break;

		case ENamedThreads::ActualRenderingThread:
			return LOCTEXT("ActualRenderingThread", "ActualRenderingThread");
			break;

		default:
			return LOCTEXT("UnknownThread", "Unknown");
		}
	}

	/**
	 * Converts the given time span in seconds to a human readable string.
	 *
	 * @param Seconds The time span to convert.
	 * @return The text representation.
	 * @todo gmp: refactor this into FText::AsTimespan or something like that
	 */
	FText TimespanToReadableText(double Seconds) const
	{
		if (Seconds < 0.0)
		{
			return LOCTEXT("Zero Length Timespan", "-");
		}

		FNumberFormattingOptions Options;
		Options.MinimumFractionalDigits = 1;
		Options.MaximumFractionalDigits = 1;

		if (Seconds < 0.0001)
		{
			return FText::Format(LOCTEXT("Seconds < 0.0001 Length Timespan", "{0} us"), FText::AsNumber(Seconds * 1000000, &Options));
		}

		if (Seconds < 0.1)
		{
			return FText::Format(LOCTEXT("Seconds < 0.1 Length Timespan", "{0} ms"), FText::AsNumber(Seconds * 1000, &Options));
		}

		if (Seconds < 60.0)
		{
			return FText::Format(LOCTEXT("Seconds < 60.0 Length Timespan", "{0} s"), FText::AsNumber(Seconds, &Options));
		}

		return LOCTEXT("> 1 minute Length Timespan", "> 1 min");
	}

private:

	/** Callback for getting the dispatch type. */
	FText HandleDispatchTypeText() const
	{
		switch (DispatchState->DispatchType)
		{
		case EMessageTracerDispatchTypes::Direct:
			return LOCTEXT("DispatchTypeDirect", "Direct");

		case EMessageTracerDispatchTypes::Pending:
			return LOCTEXT("DispatchTypePending", "Pending");

		case EMessageTracerDispatchTypes::TaskGraph:
			return LOCTEXT("DispatchTypeTaskGraph", "TaskGraph");

		default:
			return FText::GetEmpty();
		}
	}

	/** Callback for getting the dispatch type tool tip text. */
	FText HandleDispatchTypeTooltip() const
	{
		if (DispatchState->DispatchType == EMessageTracerDispatchTypes::Direct)
		{
			return LOCTEXT("DispatchDirectTooltip", "Dispatched directly (synchronously)");
		}

		if (DispatchState->DispatchType == EMessageTracerDispatchTypes::TaskGraph)
		{
			return LOCTEXT("DispatchTaskGraphTooltip", "Dispatched with Task Graph (asynchronously)");
		}

		return LOCTEXT("DispatchPendingTooltip", "Dispatch pending");
	}

	/** Callback for getting the handling time text. */
	FText HandleHandlingTimeText() const
	{
		if (DispatchState->TimeHandled > 0.0)
		{
			return TimespanToReadableText(DispatchState->TimeHandled - DispatchState->TimeDispatched);
		}

		return LOCTEXT("NotHandledYetText", "Not handled yet");
	}

	/** Callback for getting the color of a time span text. */
	FSlateColor HandleHandlingTimeColorAndOpacity() const
	{
		if (DispatchState->TimeHandled == 0.0)
		{
			return FSlateColor::UseSubduedForeground();
		}

		return FSlateColor::UseForeground();
	}

	/** Callback for getting the dispatch type image. */
	const FSlateBrush* HandleTypeImage() const
	{
		if (DispatchState->DispatchType == EMessageTracerDispatchTypes::Direct)
		{
			return Style->GetBrush("DispatchDirect");
		}

		if (DispatchState->DispatchType == EMessageTracerDispatchTypes::TaskGraph)
		{
			return Style->GetBrush("DispatchTaskGraph");
		}

		return Style->GetBrush("DispatchPending");
	}

private:

	/** Holds the message dispatch state. */
	TSharedPtr<FMessageTracerDispatchState> DispatchState;

	/** Holds a pointer to the view model. */
	TSharedPtr<FMessagingDebuggerModel> Model;

	/** Holds the widget's visual style. */
	TSharedPtr<ISlateStyle> Style;
};


#undef LOCTEXT_NAMESPACE
