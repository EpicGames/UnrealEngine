// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/MessageDetails/SMessagingMessageDetails.h"

#include "Framework/Views/TableViewMetadata.h"
#include "MessagingDebuggerModel.h"
#include "Widgets/Layout/SGridPanel.h"

#include "Widgets/MessageDetails/SMessagingDispatchStateTableRow.h"


#define LOCTEXT_NAMESPACE "SMessagingMessageDetails"


/* SMessagingMessageDetails structors
 *****************************************************************************/

SMessagingMessageDetails::~SMessagingMessageDetails()
{
	if (Model.IsValid())
	{
		Model->OnSelectedMessageChanged().RemoveAll(this);
	}
}


/* SMessagingMessageDetails interface
 *****************************************************************************/

void SMessagingMessageDetails::Construct(const FArguments& InArgs, const TSharedRef<FMessagingDebuggerModel>& InModel, const TSharedRef<ISlateStyle>& InStyle)
{
	Model = InModel;
	Style = InStyle;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.0f, 2.0f)
			[
				SNew(SGridPanel)
					.FillColumn(1, 1.0f)

				// Sender thread
				+ SGridPanel::Slot(0, 2)
					.Padding(0.0f, 4.0f, 32.0f, 4.0f)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("SenderThreadLabel", "Sender Thread:"))
					]

				+ SGridPanel::Slot(1, 2)
					.Padding(0.0f, 4.0f)
					[
						SNew(STextBlock)
						.Text(this, &SMessagingMessageDetails::HandleSenderThreadText)
					]

				// Timestamp
				+ SGridPanel::Slot(0, 3)
					.Padding(0.0f, 4.0f, 32.0f, 4.0f)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("TimestampLabel", "Timestamp:"))
					]

				+ SGridPanel::Slot(1, 3)
					.Padding(0.0f, 4.0f)
					[
						SNew(STextBlock)
							.Text(this, &SMessagingMessageDetails::HandleTimestampText)
					]

				// Expiration
				+ SGridPanel::Slot(0, 4)
					.Padding(0.0f, 4.0f, 32.0f, 4.0f)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("ExpirationLabel", "Expiration:"))
					]

				+ SGridPanel::Slot(1, 4)
					.Padding(0.0f, 4.0f)
					[
						SNew(STextBlock)
							.Text(this, &SMessagingMessageDetails::HandleExpirationText)
					]
			]

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f, 8.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
					.BorderImage(InStyle->GetBrush("GroupBorder"))
					.Padding(0.0f)
					[
						// message list
						SAssignNew(DispatchStateListView, SListView<TSharedPtr<FMessageTracerDispatchState>>)
							.ListItemsSource(&DispatchStateList)
							.SelectionMode(ESelectionMode::None)
							.OnGenerateRow(this, &SMessagingMessageDetails::HandleDispatchStateListGenerateRow)
							.HeaderRow
							(
								SNew(SHeaderRow)

								+ SHeaderRow::Column("Recipient")
									.DefaultLabel(LOCTEXT("DispatchStateListRecipientColumnHeader", "Recipient Endpoint"))
									.FillWidth(0.5f)

								+ SHeaderRow::Column("DispatchType")
									.DefaultLabel(LOCTEXT("DispatchStateListDispatchTypeColumnHeader", "Dispatch Type"))
									.FillWidth(0.25f)

								+ SHeaderRow::Column("RecipientThread")
									.DefaultLabel(LOCTEXT("DispatchStateListRecipientThreadColumnHeader", "Recipient Thread"))
									.FillWidth(0.25f)

								+ SHeaderRow::Column("DispatchLatency")
									.DefaultLabel(LOCTEXT("DispatchStateListDispatchedColumnHeader", "Dispatch Latency"))
									.FixedWidth(112.0f)
									.HAlignCell(HAlign_Right)
									.HAlignHeader(HAlign_Right)

								+ SHeaderRow::Column("HandleTime")
									.DefaultLabel(LOCTEXT("DispatchStateListHandledColumnHeader", "Handle Time"))
									.FixedWidth(80.0f)
									.HAlignCell(HAlign_Right)
									.HAlignHeader(HAlign_Right)
							)
					]
			]
	];

	Model->OnSelectedMessageChanged().AddRaw(this, &SMessagingMessageDetails::HandleModelSelectedMessageChanged);
}


/* SMessagingMessageDetails implementation
 *****************************************************************************/

void SMessagingMessageDetails::RefreshDetails()
{
	TSharedPtr<FMessageTracerMessageInfo> SelectedMessage = Model->GetSelectedMessage();

	if (SelectedMessage.IsValid())
	{
		SelectedMessage->DispatchStates.GenerateValueArray(DispatchStateList);
	}
	else
	{
		DispatchStateList.Reset();
	}

	DispatchStateListView->RequestListRefresh();
}


/* SMessagingMessageDetails event handlers
 *****************************************************************************/

TSharedRef<ITableRow> SMessagingMessageDetails::HandleDispatchStateListGenerateRow(TSharedPtr<FMessageTracerDispatchState> DispatchState, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SMessagingDispatchStateTableRow, OwnerTable, Model.ToSharedRef())
		.DispatchState(DispatchState)
		.Style(Style);
}


FText SMessagingMessageDetails::HandleExpirationText() const
{
	TSharedPtr<FMessageTracerMessageInfo> SelectedMessage = Model->GetSelectedMessage();

	if (SelectedMessage.IsValid() && SelectedMessage->Context.IsValid())
	{
		const FDateTime& Expiration = SelectedMessage->Context->GetExpiration();
		
		if (Expiration == FDateTime::MaxValue())
		{
			return LOCTEXT("ExpirationNever", "Never");
		}

		return FText::AsDateTime(Expiration);
	}

	return FText::GetEmpty();
}


void SMessagingMessageDetails::HandleModelSelectedMessageChanged()
{
	RefreshDetails();	
}


FText SMessagingMessageDetails::HandleSenderThreadText() const
{
	TSharedPtr<FMessageTracerMessageInfo> SelectedMessage = Model->GetSelectedMessage();

	if (SelectedMessage.IsValid() && SelectedMessage->Context.IsValid())
	{
		ENamedThreads::Type Thread = SelectedMessage->Context->GetSenderThread();

		switch (Thread)
		{
		case ENamedThreads::AnyThread:
			return LOCTEXT("AnyThread", "AnyThread");

		case ENamedThreads::RHIThread:
			return LOCTEXT("RHIThread", "RHIThread");

		case ENamedThreads::GameThread:
			return LOCTEXT("GameThread", "GameThread");

		case ENamedThreads::ActualRenderingThread:
			return LOCTEXT("ActualRenderingThread", "ActualRenderingThread");

		case ENamedThreads::GameThread_Local:
			return LOCTEXT("GameThread_Local", "GameThread_Local");

		case ENamedThreads::ActualRenderingThread_Local:
			return LOCTEXT("ActualRenderingThread_Local", "ActualRenderingThread_Local");

		default:
			return LOCTEXT("UnknownThread", "Unknown");
		}
	}

	return FText::GetEmpty();
}


FText SMessagingMessageDetails::HandleTimestampText() const
{
	TSharedPtr<FMessageTracerMessageInfo> SelectedMessage = Model->GetSelectedMessage();

	if (SelectedMessage.IsValid() && SelectedMessage->Context.IsValid())
	{
		return FText::AsDateTime(SelectedMessage->Context->GetTimeSent());
	}

	return FText::GetEmpty();
}


#undef LOCTEXT_NAMESPACE
