// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

template <typename ItemType> class SListView;

class FMessagingDebuggerModel;
class ISlateStyle;
class ITableRow;
class STableViewBase;

struct FMessageTracerDispatchState;


/**
 * Implements the message details panel.
 */
class SMessagingMessageDetails
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessagingMessageDetails) { }
	SLATE_END_ARGS()

public:

	/** Destructor. */
	~SMessagingMessageDetails();

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InModel The view model to use.
	 * @param InStyle The visual style to use for this widget.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FMessagingDebuggerModel>& InModel, const TSharedRef<ISlateStyle>& InStyle);

protected:

	/** Refreshes the details widget. */
	void RefreshDetails();

private:

	/** Callback for generating a row widget for the dispatch state list view. */
	TSharedRef<ITableRow> HandleDispatchStateListGenerateRow(TSharedPtr<FMessageTracerDispatchState> DispatchState, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback for getting the text of the 'Expiration' field. */
	FText HandleExpirationText() const;

	/** Callback for handling message selection changes. */
	void HandleModelSelectedMessageChanged();

	/** Callback for getting the text of the 'Sender Thread' field. */
	FText HandleSenderThreadText() const;

	/** Callback for getting the text of the 'Timestamp' field. */
	FText HandleTimestampText() const;

private:

	/** Holds the list of dispatch states. */
	TArray<TSharedPtr<FMessageTracerDispatchState>> DispatchStateList;

	/** Holds the dispatch state list view. */
	TSharedPtr<SListView<TSharedPtr<FMessageTracerDispatchState>> > DispatchStateListView;

	/** Holds a pointer to the view model. */
	TSharedPtr<FMessagingDebuggerModel> Model;

	/** Holds the widget's visual style. */
	TSharedPtr<ISlateStyle> Style;
};
