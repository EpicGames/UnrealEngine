// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SMessagingDebugger.h"

#include "IMessageTracer.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Docking/SDockTab.h"

#include "Models/MessagingDebuggerCommands.h"
#include "Models/MessagingDebuggerModel.h"
#include "Widgets/Breakpoints/SMessagingBreakpoints.h"
#include "Widgets/EndpointDetails/SMessagingEndpointDetails.h"
#include "Widgets/Endpoints/SMessagingEndpoints.h"
#include "Widgets/Graph/SMessagingGraph.h"
#include "Widgets/History/SMessagingHistory.h"
#include "Widgets/Interceptors/SMessagingInterceptors.h"
#include "Widgets/MessageData/SMessagingMessageData.h"
#include "Widgets/MessageDetails/SMessagingMessageDetails.h"
#include "Widgets/Toolbar/SMessagingDebuggerToolbar.h"
#include "Widgets/Types/SMessagingTypes.h"


#define LOCTEXT_NAMESPACE "SMessagingDebugger"


/* Local constants
 *****************************************************************************/

static const FName BreakpointsTabId("BreakpointsList");
static const FName EndpointDetailsTabId("EndpointDetails");
static const FName EndpointsTabId("EndpointList");
static const FName InteractionGraphTabId("InteractionGraph");
static const FName InterceptorsTabId("InterceptorList");
static const FName MessageDataTabId("MessageData");
static const FName MessageDetailsTabId("MessageDetails");
static const FName MessageHistoryTabId("MessageHistory");
static const FName MessageTypesTabId("MessageTypes");
static const FName ToolbarTabId("Toolbar");


/* SMessagingDebugger constructors
 *****************************************************************************/

SMessagingDebugger::SMessagingDebugger()
	: CommandList(MakeShareable(new FUICommandList))
	, MessageTracer(nullptr)
	, Model(MakeShareable(new FMessagingDebuggerModel()))
{ }


/* SMessagingDebugger interface
 *****************************************************************************/

void SMessagingDebugger::Construct(
	const FArguments& InArgs,
	const TSharedRef<SDockTab>& ConstructUnderMajorTab,
	const TSharedPtr<SWindow>& ConstructUnderWindow,
	const TSharedRef<IMessageTracer, ESPMode::ThreadSafe>& InMessageTracer,
	const TSharedRef<ISlateStyle>& InStyle
)
{
	MessageTracer = InMessageTracer;
	Style = InStyle;

	// bind commands
	FUICommandList& ActionList = *CommandList;
	{
		const FMessagingDebuggerCommands& Commands = FMessagingDebuggerCommands::Get();

		ActionList.MapAction(Commands.BreakDebugger, FExecuteAction::CreateRaw(this, &SMessagingDebugger::HandleBreakDebuggerCommandExecute), FCanExecuteAction::CreateRaw(this, &SMessagingDebugger::HandleBreakDebuggerCommandCanExecute));
		ActionList.MapAction(Commands.ClearHistory, FExecuteAction::CreateRaw(this, &SMessagingDebugger::HandleClearHistoryCommandExecute), FCanExecuteAction::CreateRaw(this, &SMessagingDebugger::HandleClearHistoryCommandCanExecute));
		ActionList.MapAction(Commands.ContinueDebugger, FExecuteAction::CreateRaw(this, &SMessagingDebugger::HandleContinueDebuggerCommandExecute), FCanExecuteAction::CreateRaw(this, &SMessagingDebugger::HandleContinueDebuggerCommandCanExecute), FIsActionChecked(), FIsActionButtonVisible::CreateRaw(this, &SMessagingDebugger::HandleContinueDebuggerCommandIsVisible));
		ActionList.MapAction(Commands.StartDebugger, FExecuteAction::CreateRaw(this, &SMessagingDebugger::HandleStartDebuggerCommandExecute), FCanExecuteAction::CreateRaw(this, &SMessagingDebugger::HandleStartDebuggerCommandCanExecute), FIsActionChecked(), FIsActionButtonVisible::CreateRaw(this, &SMessagingDebugger::HandleStartDebuggerCommandIsVisible));
		ActionList.MapAction(Commands.StepDebugger, FExecuteAction::CreateRaw(this, &SMessagingDebugger::HandleStepDebuggerCommandExecute), FCanExecuteAction::CreateRaw(this, &SMessagingDebugger::HandleStepDebuggerCommandCanExecute));
		ActionList.MapAction(Commands.StopDebugger, FExecuteAction::CreateRaw(this, &SMessagingDebugger::HandleStopDebuggerCommandExecute), FCanExecuteAction::CreateRaw(this, &SMessagingDebugger::HandleStopDebuggerCommandCanExecute));
	}

	// create & initialize tab manager
	TabManager = FGlobalTabmanager::Get()->NewTabManager(ConstructUnderMajorTab);
	{
		TSharedRef<FWorkspaceItem> AppMenuGroup = TabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("MessagingDebuggerGroupName", "Messaging Debugger"));

		TabManager->RegisterTabSpawner(BreakpointsTabId, FOnSpawnTab::CreateRaw(this, &SMessagingDebugger::HandleTabManagerSpawnTab, BreakpointsTabId))
			.SetDisplayName(LOCTEXT("BreakpointsTabTitle", "Breakpoints"))
			.SetGroup(AppMenuGroup)
			.SetIcon(FSlateIcon(Style->GetStyleSetName(), "BreakpointsTabIcon"));

		TabManager->RegisterTabSpawner(EndpointDetailsTabId, FOnSpawnTab::CreateRaw(this, &SMessagingDebugger::HandleTabManagerSpawnTab, EndpointDetailsTabId))
			.SetDisplayName(LOCTEXT("EndpointDetailsTabTitle", "Endpoint Details"))
			.SetGroup(AppMenuGroup)
			.SetIcon(FSlateIcon(Style->GetStyleSetName(), "EndpointDetailsTabIcon"));

		TabManager->RegisterTabSpawner(EndpointsTabId, FOnSpawnTab::CreateRaw(this, &SMessagingDebugger::HandleTabManagerSpawnTab, EndpointsTabId))
			.SetDisplayName(LOCTEXT("EndpointsTabTitle", "Endpoints"))
			.SetGroup(AppMenuGroup)
			.SetIcon(FSlateIcon(Style->GetStyleSetName(), "EndpointsTabIcon"));

		TabManager->RegisterTabSpawner(InteractionGraphTabId, FOnSpawnTab::CreateRaw(this, &SMessagingDebugger::HandleTabManagerSpawnTab, InteractionGraphTabId))
			.SetDisplayName(LOCTEXT("InteractionGraphTabTitle", "Interaction Graph"))
			.SetGroup(AppMenuGroup)
			.SetIcon(FSlateIcon(Style->GetStyleSetName(), "InteractionGraphTabIcon"));

		TabManager->RegisterTabSpawner(InterceptorsTabId, FOnSpawnTab::CreateRaw(this, &SMessagingDebugger::HandleTabManagerSpawnTab, InterceptorsTabId))
			.SetDisplayName(LOCTEXT("InterceptorsTabTitle", "Interceptors"))
			.SetGroup(AppMenuGroup)
			.SetIcon(FSlateIcon(Style->GetStyleSetName(), "InterceptorsTabIcon"));

		TabManager->RegisterTabSpawner(MessageDataTabId, FOnSpawnTab::CreateRaw(this, &SMessagingDebugger::HandleTabManagerSpawnTab, MessageDataTabId))
			.SetDisplayName(LOCTEXT("MessageDataTabTitle", "Message Data"))
			.SetGroup(AppMenuGroup)
			.SetIcon(FSlateIcon(Style->GetStyleSetName(), "MessageDataTabIcon"));

		TabManager->RegisterTabSpawner(MessageDetailsTabId, FOnSpawnTab::CreateRaw(this, &SMessagingDebugger::HandleTabManagerSpawnTab, MessageDetailsTabId))
			.SetDisplayName(LOCTEXT("MessageDetailsTabTitle", "Message Details"))
			.SetGroup(AppMenuGroup)
			.SetIcon(FSlateIcon(Style->GetStyleSetName(), "MessageDetailsTabIcon"));

		TabManager->RegisterTabSpawner(MessageHistoryTabId, FOnSpawnTab::CreateRaw(this, &SMessagingDebugger::HandleTabManagerSpawnTab, MessageHistoryTabId))
			.SetDisplayName(LOCTEXT("MessageHistoryTabTitle", "Message History"))
			.SetGroup(AppMenuGroup)
			.SetIcon(FSlateIcon(Style->GetStyleSetName(), "MessageHistoryTabIcon"));

		TabManager->RegisterTabSpawner(MessageTypesTabId, FOnSpawnTab::CreateRaw(this, &SMessagingDebugger::HandleTabManagerSpawnTab, MessageTypesTabId))
			.SetDisplayName(LOCTEXT("MessageTypesTabTitle", "Message Types"))
			.SetGroup(AppMenuGroup)
			.SetIcon(FSlateIcon(Style->GetStyleSetName(), "MessageTypesTabIcon"));

		TabManager->RegisterTabSpawner(ToolbarTabId, FOnSpawnTab::CreateRaw(this, &SMessagingDebugger::HandleTabManagerSpawnTab, ToolbarTabId))
			.SetDisplayName(LOCTEXT("ToolbarTabTitle", "Toolbar"))
			.SetGroup(AppMenuGroup)
			.SetIcon(FSlateIcon(Style->GetStyleSetName(), "ToolbarTabIcon"));
	}

	// create tab layout
	const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("MessagingDebuggerLayout_v1.0")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					// left column
					FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.25f)
						->Split
						(
							FTabManager::NewStack()
								->AddTab(EndpointsTabId, ETabState::OpenedTab)
								->SetSizeCoefficient(0.5f)
						)
						->Split
						(
							FTabManager::NewStack()
								->AddTab(EndpointDetailsTabId, ETabState::OpenedTab)
								->AddTab(InterceptorsTabId, ETabState::OpenedTab)
								->SetForegroundTab(EndpointDetailsTabId)
								->SetSizeCoefficient(0.5f)
						)
				)
				->Split
				(
					// center column
					FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.5f)
						->Split
						(
							FTabManager::NewStack()
								->AddTab(ToolbarTabId, ETabState::OpenedTab)
								->SetHideTabWell(true)
						)/*
						->Split
						(
							FTabManager::NewStack()
								->AddTab(InteractionGraphTabName, ETabState::OpenedTab)
								->SetHideTabWell(true)
						)*/
						->Split
						(
							FTabManager::NewStack()
								->AddTab(MessageHistoryTabId, ETabState::OpenedTab)
								->SetHideTabWell(true)
								->SetSizeCoefficient(0.725f)
						)
						->Split
						(
							FTabManager::NewStack()
								->AddTab(BreakpointsTabId, ETabState::OpenedTab)
								->AddTab(MessageDetailsTabId, ETabState::OpenedTab)
								->SetSizeCoefficient(0.275f)
						)
				)
				->Split
				(
					// right column
					FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.25f)
						->Split
						(
							FTabManager::NewStack()
								->AddTab(MessageTypesTabId, ETabState::OpenedTab)
								->SetSizeCoefficient(0.5f)
						)
						->Split
						(
							FTabManager::NewStack()
								->AddTab(MessageDataTabId, ETabState::OpenedTab)
								->SetForegroundTab(MessageDetailsTabId)
								->SetSizeCoefficient(0.5f)
						)
				)
		);

	// create & initialize main menu
	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(TSharedPtr<FUICommandList>());

	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("WindowMenuLabel", "Window"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateStatic(&SMessagingDebugger::FillWindowMenu, TabManager),
		"Window"
	);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				MenuBarBuilder.MakeWidget()
			]
	
		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				TabManager->RestoreFrom(Layout, ConstructUnderWindow).ToSharedRef()
			]
	];

	ConstructUnderMajorTab->SetOnPersistVisualState(SDockTab::FOnPersistVisualState::CreateRaw(this, &SMessagingDebugger::HandleMajorTabPersistVisualState));
}


/* SMessagingDebugger implementation
 *****************************************************************************/

void SMessagingDebugger::FillWindowMenu(FMenuBuilder& MenuBuilder, const TSharedPtr<FTabManager> TabManager)
{
	if (!TabManager.IsValid())
	{
		return;
	}

	TabManager->PopulateLocalTabSpawnerMenu(MenuBuilder);
}


/* SWidget overrides
 *****************************************************************************/

FReply SMessagingDebugger::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return CommandList->ProcessCommandBindings(InKeyEvent)
		? FReply::Handled()
		: FReply::Unhandled();
}


bool SMessagingDebugger::SupportsKeyboardFocus() const
{
	return true;
}


/* SMessagingDebugger callbacks
 *****************************************************************************/

bool SMessagingDebugger::HandleBreakDebuggerCommandCanExecute() const
{
	return MessageTracer->IsRunning() && !MessageTracer->IsBreaking();
}


void SMessagingDebugger::HandleBreakDebuggerCommandExecute()
{
	MessageTracer->Break();
}


bool SMessagingDebugger::HandleClearHistoryCommandCanExecute() const
{
	return MessageTracer->HasMessages();
}


void SMessagingDebugger::HandleClearHistoryCommandExecute()
{
	MessageTracer->Reset();
}


bool SMessagingDebugger::HandleContinueDebuggerCommandCanExecute() const
{
	return (!MessageTracer->IsRunning() || MessageTracer->IsBreaking());
}


void SMessagingDebugger::HandleContinueDebuggerCommandExecute()
{
	MessageTracer->Continue();
}


bool SMessagingDebugger::HandleContinueDebuggerCommandIsVisible() const
{
	return MessageTracer->IsBreaking();
}


void SMessagingDebugger::HandleMajorTabPersistVisualState()
{
	// save any settings here
}


bool SMessagingDebugger::HandleStartDebuggerCommandCanExecute() const
{
	return !MessageTracer->IsRunning();
}


void SMessagingDebugger::HandleStartDebuggerCommandExecute()
{
	MessageTracer->Continue();
}


bool SMessagingDebugger::HandleStartDebuggerCommandIsVisible() const
{
	return !MessageTracer->IsBreaking();
}


bool SMessagingDebugger::HandleStepDebuggerCommandCanExecute() const
{
	return MessageTracer->IsBreaking();
}


void SMessagingDebugger::HandleStepDebuggerCommandExecute()
{
	MessageTracer->Step();
}


bool SMessagingDebugger::HandleStopDebuggerCommandCanExecute() const
{
	return MessageTracer->IsRunning();
}


void SMessagingDebugger::HandleStopDebuggerCommandExecute()
{
	MessageTracer->Stop();
}


TSharedRef<SDockTab> SMessagingDebugger::HandleTabManagerSpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier) const
{
	TSharedPtr<SWidget> TabWidget = SNullWidget::NullWidget;
	bool AutoSizeTab = false;

	if (TabIdentifier == BreakpointsTabId)
	{
		TabWidget = SNew(SMessagingBreakpoints, Style.ToSharedRef(), MessageTracer.ToSharedRef());
	}
	else if (TabIdentifier == EndpointDetailsTabId)
	{
		TabWidget = SNew(SMessagingEndpointDetails, Model, Style.ToSharedRef());
	}
	else if (TabIdentifier == EndpointsTabId)
	{
		TabWidget = SNew(SMessagingEndpoints, Model, Style.ToSharedRef(), MessageTracer.ToSharedRef());
	}
	else if (TabIdentifier == InteractionGraphTabId)
	{
		TabWidget = SNew(SMessagingGraph, Style.ToSharedRef());
	}
	else if (TabIdentifier == InterceptorsTabId)
	{
		TabWidget = SNew(SMessagingInterceptors, Model, Style.ToSharedRef(), MessageTracer.ToSharedRef());
	}
	else if (TabIdentifier == MessageDataTabId)
	{
		TabWidget = SNew(SMessagingMessageData, Model, Style.ToSharedRef());
	}
	else if (TabIdentifier == MessageDetailsTabId)
	{
		TabWidget = SNew(SMessagingMessageDetails, Model, Style.ToSharedRef());
	}
	else if (TabIdentifier == MessageHistoryTabId)
	{
		TabWidget = SNew(SMessagingHistory, Model, Style.ToSharedRef(), MessageTracer.ToSharedRef());
	}
	else if (TabIdentifier == MessageTypesTabId)
	{
		TabWidget = SNew(SMessagingTypes, Model, Style.ToSharedRef(), MessageTracer.ToSharedRef());
	}
	else if (TabIdentifier == ToolbarTabId)
	{
		TabWidget = SNew(SMessagingDebuggerToolbar, Style.ToSharedRef(), CommandList);
		AutoSizeTab = true;
	}

	return SNew(SDockTab)
		.ShouldAutosize(AutoSizeTab)
		.TabRole(ETabRole::PanelTab)
		[
			TabWidget.ToSharedRef()
		];
}


#undef LOCTEXT_NAMESPACE
