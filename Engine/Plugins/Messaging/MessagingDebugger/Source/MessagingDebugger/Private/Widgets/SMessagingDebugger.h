// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ISlateStyle;
class SDockTab;

class FMenuBuilder;
class FMessagingDebuggerModel;
class FSpawnTabArgs;
class FTabManager;
class FUICommandList;
class IMessageTracer;
class SWindow;


/**
 * Implements the messaging system debugger application.
 */
class SMessagingDebugger
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessagingDebugger) { }
	SLATE_END_ARGS()

public:

	/** Default constructor. */
	SMessagingDebugger();

public:

	/**
	 * Constructs the application.
	 *
	 * @param InArgs The Slate argument list.
	 * @param ConstructUnderMajorTab The major tab which will contain the session front-end.
	 * @param ConstructUnderWindow The window in which this widget is being constructed.
	 * @param InMessageTracer The message tracer to use for the debugger.
	 * @param InStyleSet The style set to use.
	 */
	void Construct(
		const FArguments& InArgs,
		const TSharedRef<SDockTab>& ConstructUnderMajorTab,
		const TSharedPtr<SWindow>& ConstructUnderWindow,
		const TSharedRef<IMessageTracer, ESPMode::ThreadSafe>& InMessageTracer,
		const TSharedRef<ISlateStyle>& InStyle);

protected:

	/**
	 * Fills the Window menu with menu items.
	 *
	 * @param MenuBuilder The multi-box builder that should be filled with content for this pull-down menu.
	 * @param TabManager A Tab Manager from which to populate tab spawner menu items.
	 */
	static void FillWindowMenu(FMenuBuilder& MenuBuilder, const TSharedPtr<FTabManager> TabManager);

protected:

	//~ SWidget overrides

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual bool SupportsKeyboardFocus() const override;

private:

	/** Callback for checking whether the BreakDebugger command can be executed. */
	bool HandleBreakDebuggerCommandCanExecute() const;

	/** Callback for clicking the BreakDebugger command. */
	void HandleBreakDebuggerCommandExecute();

	/** Callback for checking whether the ClearHistory command can be executed. */
	bool HandleClearHistoryCommandCanExecute() const;

	/** Callback for clicking the ClearHistory command. */
	void HandleClearHistoryCommandExecute();

	/** Callback for checking whether the ContinueDebugger command can be executed. */
	bool HandleContinueDebuggerCommandCanExecute() const;

	/** Callback for clicking the ContinueDebugger command. */
	void HandleContinueDebuggerCommandExecute();

	/** Callback for checking whether the ContinueDebugger command is visible. */
	bool HandleContinueDebuggerCommandIsVisible() const;

	/** Callback for for when the owner tab's visual state is being persisted. */
	void HandleMajorTabPersistVisualState();

	/** Callback for checking whether the StartDebugger command can be executed. */
	bool HandleStartDebuggerCommandCanExecute() const;

	/** Callback for clicking the StartDebugger command. */
	void HandleStartDebuggerCommandExecute();

	/** Callback for checking whether the StartDebugger command is visible. */
	bool HandleStartDebuggerCommandIsVisible() const;

	/** Callback for checking whether the StepDebugger command can be executed. */
	bool HandleStepDebuggerCommandCanExecute() const;

	/** Callback for clicking the StepDebugger command. */
	void HandleStepDebuggerCommandExecute();

	/** Callback for checking whether the StopDebugger command can be executed. */
	bool HandleStopDebuggerCommandCanExecute() const;

	/** Callback for clicking the StopDebugger command. */
	void HandleStopDebuggerCommandExecute();

	/** Callback for spawning tabs. */
	TSharedRef<SDockTab> HandleTabManagerSpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier) const;

private:

	/** Holds the list of UI commands. */
	TSharedRef<FUICommandList> CommandList;

	/** Holds a pointer to the message bus tracer. */
	TSharedPtr<IMessageTracer, ESPMode::ThreadSafe> MessageTracer;

	/** Holds the messaging debugger's view model. */
	TSharedRef<FMessagingDebuggerModel> Model;

	/** Holds the widget's visual style. */
	TSharedPtr<ISlateStyle> Style;

	/** Holds the tab manager that manages the front-end's tabs. */
	TSharedPtr<FTabManager> TabManager;
};
