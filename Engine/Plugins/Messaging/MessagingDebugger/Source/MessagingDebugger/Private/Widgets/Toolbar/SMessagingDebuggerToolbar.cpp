// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Toolbar/SMessagingDebuggerToolbar.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "Models/MessagingDebuggerCommands.h"


#define LOCTEXT_NAMESPACE "SMessagingDebuggerToolbar"


/* SMessagingDebuggerToolbar interface
 *****************************************************************************/

void SMessagingDebuggerToolbar::Construct(const FArguments& InArgs, const TSharedRef<ISlateStyle>& InStyle, const TSharedRef<FUICommandList>& InCommandList)
{
	ChildSlot
	[
		MakeToolbar(InCommandList)
	];
}


/* SMessagingDebuggerToolbar implementation
 *****************************************************************************/

TSharedRef<SWidget> SMessagingDebuggerToolbar::MakeToolbar(const TSharedRef<FUICommandList>& CommandList)
{
	FToolBarBuilder ToolBarBuilder(CommandList, FMultiBoxCustomization::None);

	ToolBarBuilder.BeginSection("Debugger");
	{
		ToolBarBuilder.AddToolBarButton(FMessagingDebuggerCommands::Get().StartDebugger, NAME_None, LOCTEXT("StartDebugger", "Start"));
		ToolBarBuilder.AddToolBarButton(FMessagingDebuggerCommands::Get().ContinueDebugger, NAME_None, LOCTEXT("ContinueDebugger", "Continue"));
		ToolBarBuilder.AddToolBarButton(FMessagingDebuggerCommands::Get().StepDebugger, NAME_None, LOCTEXT("StepDebugger", "Step"));
		ToolBarBuilder.AddToolBarButton(FMessagingDebuggerCommands::Get().BreakDebugger, NAME_None, LOCTEXT("BreakAtNextMessage", "Break"));
		ToolBarBuilder.AddToolBarButton(FMessagingDebuggerCommands::Get().StopDebugger,  NAME_None, LOCTEXT("StopDebugger", "Stop"));

		ToolBarBuilder.AddSeparator();
		ToolBarBuilder.AddToolBarButton(FMessagingDebuggerCommands::Get().ClearHistory, NAME_None, LOCTEXT("ClearHistory", "Clear History"));
	}

	return ToolBarBuilder.MakeWidget();
}


#undef LOCTEXT_NAMESPACE
