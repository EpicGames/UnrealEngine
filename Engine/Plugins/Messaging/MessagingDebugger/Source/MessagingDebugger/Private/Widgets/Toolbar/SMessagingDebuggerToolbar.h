// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FUICommandList;
class ISlateStyle;
class SWidget;


/**
 * Implements the messaging debugger toolbar widget.
 */
class SMessagingDebuggerToolbar
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessagingDebuggerToolbar) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InStyle The visual style to use for this widget.
	 * @param InCommandList The command list to bind to.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<ISlateStyle>& InStyle, const TSharedRef<FUICommandList>& InCommandList);

protected:

	/**
	 * Creates the toolbar widget.
	 *
	 * @param CommandList The command list to use for the toolbar buttons.
	 * @return The toolbar widget.
	 */
	TSharedRef<SWidget> MakeToolbar(const TSharedRef<FUICommandList>& CommandList);
};
