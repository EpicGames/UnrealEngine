// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Types/SMessagingTypes.h"

#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Layout/SExpandableArea.h"

#include "Models/MessagingDebuggerModel.h"
#include "Models/MessagingDebuggerTypeFilter.h"
#include "Widgets/Types/SMessagingTypesFilterBar.h"
#include "Widgets/Types/SMessagingTypesTableRow.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SMessagingTypes"


/* SMessagingTypes structors
 *****************************************************************************/

SMessagingTypes::~SMessagingTypes()
{
	if (Model.IsValid())
	{
		Model->OnSelectedMessageChanged().RemoveAll(this);
	}

	if (Tracer.IsValid())
	{
		Tracer->OnMessagesReset().RemoveAll(this);
		Tracer->OnTypeAdded().RemoveAll(this);
	}
}


/* SMessagingTypes interface
 *****************************************************************************/

void SMessagingTypes::Construct(const FArguments& InArgs, const TSharedRef<FMessagingDebuggerModel>& InModel, const TSharedRef<ISlateStyle>& InStyle, const TSharedRef<IMessageTracer, ESPMode::ThreadSafe>& InTracer)
{
	Filter = MakeShareable(new FMessagingDebuggerTypeFilter());
	Model = InModel;
	Style = InStyle;
	Tracer = InTracer;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SExpandableArea)
					.AreaTitle(LOCTEXT("TypeFilterAreaTitle", "Message Type Filter"))
					.InitiallyCollapsed(true)
					.Padding(8.0f)
					.BodyContent()
					[
						// filter bar
						SNew(SMessagingTypesFilterBar, Filter.ToSharedRef())
					]
			]

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f, 4.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
					.BorderImage(InStyle->GetBrush("GroupBorder"))
					.Padding(0.0f)
					[
						// type list
						SAssignNew(TypeListView, SListView<TSharedPtr<FMessageTracerTypeInfo>>)
							.ListItemsSource(&TypeList)
							.SelectionMode(ESelectionMode::Multi)
							.OnGenerateRow(this, &SMessagingTypes::HandleTypeListGenerateRow)
							.OnSelectionChanged(this, &SMessagingTypes::HandleTypeListSelectionChanged)
							.HeaderRow
							(
								SNew(SHeaderRow)

								+ SHeaderRow::Column("Break")
									.FixedWidth(24.0f)
									.HAlignHeader(HAlign_Center)
									.VAlignHeader(VAlign_Center)
									.HeaderContent()
									[
										SNew(SImage)
											.Image(InStyle->GetBrush("BreakColumn"))
											.ToolTipText(LOCTEXT("EndpointListBreakpointsColumnTooltip", "Breakpoints"))
									]

								+ SHeaderRow::Column("Name")
									.DefaultLabel(LOCTEXT("TypeListNameColumnHeader", "Name"))
									.FillWidth(0.9f)

								+ SHeaderRow::Column("Messages")
									.FillWidth(0.1f)
									.HAlignCell(HAlign_Right)
									.HAlignHeader(HAlign_Right)
									.VAlignHeader(VAlign_Center)
									.HeaderContent()
									[
										SNew(SImage)
											.Image(Style->GetBrush("SentMessage"))
											.ToolTipText(LOCTEXT("TypeListMessagesColumnTooltip", "Number of messages per message type"))
									]

								+ SHeaderRow::Column("Visibility")
									.FixedWidth(26.0f)
									.HAlignCell(HAlign_Center)
									.HAlignHeader(HAlign_Center)
									.VAlignHeader(VAlign_Center)
									.HeaderContent()
									[
										SNew(SImage)
											.Image(Style->GetBrush("Visibility"))
											.ToolTipText(LOCTEXT("TypeListVisibilityColumnTooltip", "Visibility"))
									]
							)
					]
			]
	];

	Filter->OnChanged().AddRaw(this, &SMessagingTypes::HandleFilterChanged);
	Model->OnSelectedMessageChanged().AddRaw(this, &SMessagingTypes::HandleModelSelectedMessageChanged);
	Tracer->OnMessagesReset().AddRaw(this, &SMessagingTypes::HandleTracerMessagesReset);
	Tracer->OnTypeAdded().AddRaw(this, &SMessagingTypes::HandleTracerTypeAdded);

	ReloadTypes();
}


/* SMessagingTypes implementation
 *****************************************************************************/

void SMessagingTypes::AddType(const TSharedRef<FMessageTracerTypeInfo>& TypeInfo)
{
	if (Filter->FilterType(TypeInfo))
	{
		TypeList.Add(TypeInfo);
		TypeListView->RequestListRefresh();
	}
}


void SMessagingTypes::ReloadTypes()
{
	TypeList.Reset();
	
	TArray<TSharedPtr<FMessageTracerTypeInfo>> Types;

	if (Tracer->GetMessageTypes(Types) > 0)
	{
		for (TArray<TSharedPtr<FMessageTracerTypeInfo>>::TConstIterator It(Types); It; ++It)
		{
			AddType(It->ToSharedRef());
		}
	}

	TypeListView->RequestListRefresh();
}


/* SMessagingTypes callbacks
 *****************************************************************************/

void SMessagingTypes::HandleFilterChanged()
{
	ReloadTypes();
}


void SMessagingTypes::HandleModelSelectedMessageChanged()
{
	TSharedPtr<FMessageTracerMessageInfo> SelectedMessage = Model->GetSelectedMessage();

	if (SelectedMessage.IsValid())
	{
		TypeListView->SetSelection(SelectedMessage->TypeInfo);
	}
}


void SMessagingTypes::HandleTracerMessagesReset()
{
	ReloadTypes();
}


void SMessagingTypes::HandleTracerTypeAdded(TSharedRef<FMessageTracerTypeInfo> TypeInfo)
{
	AddType(TypeInfo);
}


TSharedRef<ITableRow> SMessagingTypes::HandleTypeListGenerateRow(TSharedPtr<FMessageTracerTypeInfo> TypeInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SMessagingTypesTableRow, OwnerTable, Model.ToSharedRef())
		.HighlightText(this, &SMessagingTypes::HandleTypeListGetHighlightText)
		.Style(Style)
		.TypeInfo(TypeInfo);
}


FText SMessagingTypes::HandleTypeListGetHighlightText() const
{
	return FText::GetEmpty();
	//return FilterBar->GetFilterText();
}


void SMessagingTypes::HandleTypeListSelectionChanged(TSharedPtr<FMessageTracerTypeInfo> InItem, ESelectInfo::Type SelectInfo)
{

}


#undef LOCTEXT_NAMESPACE
