// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

namespace ESelectInfo { enum Type : int; }
template <typename ItemType> class SListView;

class FMessagingDebuggerModel;
class FMessagingDebuggerTypeFilter;
class IMessageTracer;
class ISlateStyle;
class ITableRow;
class STableViewBase;

struct FMessageTracerTypeInfo;


/**
 * Implements the message types panel.
 */
class SMessagingTypes
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessagingTypes) { }
	SLATE_END_ARGS()

public:

	/** Destructor. */
	~SMessagingTypes();

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InModel The view model to use.
	 * @param InStyle The visual style to use for this widget.
	 * @param InTracer The message tracer.
	 */
	void Construct(
		const FArguments& InArgs,
		const TSharedRef<FMessagingDebuggerModel>& InModel,
		const TSharedRef<ISlateStyle>& InStyle,
		const TSharedRef<IMessageTracer, ESPMode::ThreadSafe>& InTracer);

protected:

	/**
	 * Adds the given message to the history.
	 *
	 * @param TypeInfo The information about the message type to add.
	 */
	void AddType(const TSharedRef<FMessageTracerTypeInfo>& TypeInfo);

	/** Reloads the message history. */
	void ReloadTypes();

private:

	/** Callback for message type filter changes. */
	void HandleFilterChanged();

	/** Callback for handling message selection changes. */
	void HandleModelSelectedMessageChanged();

	/** Callback for when the tracer's message history has been reset. */
	void HandleTracerMessagesReset();

	/** Callback for when a message type has been added to the message tracer. */
	void HandleTracerTypeAdded(TSharedRef<FMessageTracerTypeInfo> TypeInfo);

	/** Callback for generating a row widget for the message type list view. */
	TSharedRef<ITableRow> HandleTypeListGenerateRow(TSharedPtr<FMessageTracerTypeInfo> TypeInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback for getting the highlight string for message types. */
	FText HandleTypeListGetHighlightText() const;

	/** Callback for selecting message types. */
	void HandleTypeListSelectionChanged(TSharedPtr<FMessageTracerTypeInfo> InItem, ESelectInfo::Type SelectInfo);

private:

	/** Holds the message type filter model. */
	TSharedPtr<FMessagingDebuggerTypeFilter> Filter;

	/** Holds a pointer to the view model. */
	TSharedPtr<FMessagingDebuggerModel> Model;

	/** Holds the widget's visual style. */
	TSharedPtr<ISlateStyle> Style;

	/** Holds a pointer to the message bus tracer. */
	TSharedPtr<IMessageTracer, ESPMode::ThreadSafe> Tracer;

	/** Holds the filtered list of message types. */
	TArray<TSharedPtr<FMessageTracerTypeInfo>> TypeList;

	/** Holds the message type list view. */
	TSharedPtr<SListView<TSharedPtr<FMessageTracerTypeInfo>>> TypeListView;
};
