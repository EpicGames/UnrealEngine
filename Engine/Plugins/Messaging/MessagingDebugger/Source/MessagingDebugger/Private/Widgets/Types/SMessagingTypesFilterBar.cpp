// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Types/SMessagingTypesFilterBar.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SSearchBox.h"

#include "Models/MessagingDebuggerTypeFilter.h"


#define LOCTEXT_NAMESPACE "SMessagingTypesFilterBar"


/* SMessagingTypesFilterBar interface
*****************************************************************************/

void SMessagingTypesFilterBar::Construct(const FArguments& InArgs, TSharedRef<FMessagingDebuggerTypeFilter> InFilter)
{
	Filter = InFilter;

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Top)
			[
				// search box
				SNew(SSearchBox)
					.HintText(LOCTEXT("SearchBoxHint", "Search message types"))
					.OnTextChanged_Lambda([this](const FText& NewText) {
						Filter->SetFilterString(NewText.ToString());
					})
			]
	];
}

#undef LOCTEXT_NAMESPACE
