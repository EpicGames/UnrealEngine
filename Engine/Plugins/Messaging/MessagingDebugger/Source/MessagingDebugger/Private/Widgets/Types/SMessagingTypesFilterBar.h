// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FMessagingDebuggerTypeFilter;


/**
 * Implements the message type list filter bar widget.
 */
class SMessagingTypesFilterBar
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMessagingTypesFilterBar) { }

		/** Called when the filter settings have changed. */
		SLATE_EVENT(FSimpleDelegate, OnFilterChanged)

	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InFilter The filter model.
	 */
	void Construct(const FArguments& InArgs, TSharedRef<FMessagingDebuggerTypeFilter> InFilter);

private:

	/** Holds the filter model. */
	TSharedPtr<FMessagingDebuggerTypeFilter> Filter;
};
