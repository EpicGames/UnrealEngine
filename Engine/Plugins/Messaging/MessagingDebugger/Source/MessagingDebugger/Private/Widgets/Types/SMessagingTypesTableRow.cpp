// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Types/SMessagingTypesTableRow.h"

#include "IMessageTracer.h"

#include "Models/MessagingDebuggerModel.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SMessagingTypesTableRow"


/* SMessagingTypesTableRow interface
 *****************************************************************************/

void SMessagingTypesTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FMessagingDebuggerModel>& InModel)
{
	check(InArgs._Style.IsValid());
	check(InArgs._TypeInfo.IsValid());

	HighlightText = InArgs._HighlightText;
	Model = InModel;
	Style = InArgs._Style;
	TypeInfo = InArgs._TypeInfo;

	SMultiColumnTableRow<TSharedPtr<FMessageTracerTypeInfo>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}


/* SMultiColumnTableRow interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SMessagingTypesTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == "Break")
	{
		return SNew(SBorder)
			.BorderBackgroundColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.33f))
			.BorderImage(Style->GetBrush("BreakpointBorder"));

/*			return SNew(SImage)
			.Image(Style->GetBrush("BreakDisabled"));*/
	}
	else if (ColumnName == "Messages")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.HighlightText(HighlightText)
					.Text_Lambda([this]() {
						return FText::AsNumber(TypeInfo->Messages.Num());
					})
			];
	}
	else if (ColumnName == "Name")
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.HighlightText(HighlightText)
					.Text(FText::FromName(TypeInfo->TypePathName.GetAssetName()))
			];
	}
	else if (ColumnName == "Visibility")
	{
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
					.Style(&Style->GetWidgetStyle<FCheckBoxStyle>("VisibilityCheckbox"))
					.IsChecked_Lambda([this]() -> ECheckBoxState {
						return (Model->IsTypeVisible(TypeInfo.ToSharedRef()))
							? ECheckBoxState::Checked
							: ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([this](ECheckBoxState CheckState) {
						Model->SetTypeVisibility(TypeInfo.ToSharedRef(), (CheckState == ECheckBoxState::Checked));
					})
					.ToolTipText(LOCTEXT("VisibilityCheckboxTooltipText", "Toggle visibility of messages of this type"))
			];
	}

	return SNullWidget::NullWidget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE
