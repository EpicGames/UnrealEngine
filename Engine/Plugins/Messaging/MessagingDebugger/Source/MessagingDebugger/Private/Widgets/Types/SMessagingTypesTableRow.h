// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class STableViewBase;

class FMessagingDebuggerModel;

struct FMessageTracerTypeInfo;


/**
 * Implements a row widget for the message type list.
 */
class SMessagingTypesTableRow
	: public SMultiColumnTableRow<TSharedPtr<FMessageTracerTypeInfo>>
{
public:

	SLATE_BEGIN_ARGS(SMessagingTypesTableRow) { }
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(TSharedPtr<FMessageTracerTypeInfo>, TypeInfo)
		SLATE_ARGUMENT(TSharedPtr<ISlateStyle>, Style)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InOwnerTableView The table view that owns this row.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FMessagingDebuggerModel>& InModel);

public:

	//~ SMultiColumnTableRow interface

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	/** Holds the highlight string for the message. */
	TAttribute<FText> HighlightText;

	/** Holds a pointer to the view model. */
	TSharedPtr<FMessagingDebuggerModel> Model;

	/** Holds the widget's visual style. */
	TSharedPtr<ISlateStyle> Style;

	/** Holds message type's debug information. */
	TSharedPtr<FMessageTracerTypeInfo> TypeInfo;
};
